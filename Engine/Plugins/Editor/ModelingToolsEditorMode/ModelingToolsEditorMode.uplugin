{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Modeling Tools Editor Mode",
	"Description": "Modeling Tools Mode includes a suite of interactive tools for creating and editing meshes in the Editor",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ModelingToolsEditorMode",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "MeshModelingToolset",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "MeshLODToolset",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "ToolPresets",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "StylusInput",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		}
	]
}