// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ModelingToolsEditorMode : ModuleRules
{
	public ModelingToolsEditorMode(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"RHI",
				"Slate",
				"SlateCore",
				"Engine",
				"InputCore",
				"EditorFramework",
				"UnrealEd",
				"ContentBrowserData",
				"StatusBar",
				"Projects",
				"TypedElementRuntime",
				"InteractiveToolsFramework",
				"EditorInteractiveToolsFramework",
				"GeometryFramework",
				"GeometryCore",
				"GeometryProcessingInterfaces",
				"DynamicMesh",
				"ModelingComponents",
				"ModelingComponentsEditorOnly",
				"MeshModelingTools",
				"MeshModelingToolsExp",
				"MeshModelingToolsEditorOnly",
				"MeshModelingToolsEditorOnlyExp",
				"MeshLODToolset",
				"ToolWidgets",
				"EditorWidgets",
				"ModelingEditorUI",
				"WidgetRegistration",
				"DeveloperSettings",
				"PropertyEditor",
				"ToolMenus",
				"EditorConfig",
				"ToolPresetAsset",
				"ToolPresetEditor",
				"EditorConfig",
				"StylusInput",
				// ... add private dependencies that you statically link with here ...	
			}
		);

		PublicDefinitions.Add("WITH_PROXYLOD=" + (Target.Platform == UnrealTargetPlatform.Win64 ? '1' : '0'));
		PrivateDefinitions.Add("ENABLE_STYLUS_SUPPORT=" + (Target.Platform == UnrealTargetPlatform.Win64 ? '1' : '0'));
	}
}