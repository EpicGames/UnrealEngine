// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/MeshVertexAttributePaintToolDetailCustomization.h"

#include "MeshVertexAttributePaintToolBase.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ModelingToolsEditorModeStyle.h"
#include "SColorGradientEditor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "Framework/Commands/InputChord.h"
#include "IDocumentation.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "MeshVertexAttributePaintToolDetailCustomization"

namespace UE::MeshVertexAttributePaintToolBase
{
	// layout constants
	float FVertexAttributePaintToolDetailCustomization::WeightSliderWidths = 150.0f;
	float FVertexAttributePaintToolDetailCustomization::WeightEditingLabelsPercent = 0.40f;
	float FVertexAttributePaintToolDetailCustomization::WeightEditVerticalPadding = 4.0f;
	float FVertexAttributePaintToolDetailCustomization::WeightEditHorizontalPadding = 2.0f;

	static const FName QueryTypeInterpolated(TEXT("Interpolated"));
	static const FName QueryTypeVertexFast(TEXT("Nearest Vertex Fast"));
	static const FName QueryTypeVertexAccurate(TEXT("Nearest Vertex Accurate"));

	static const FName VisibilityFilterNone(TEXT("All"));
	static const FName VisibilityFilterUnoccluded(TEXT("Unoccluded"));

	static EMeshVertexAttributePaintToolVisibilityType GetVisibilityFilterEnumFromName(FName Name)
	{
		if (Name == VisibilityFilterUnoccluded) 
		{ 
			return EMeshVertexAttributePaintToolVisibilityType::Unoccluded;
		}
		// VisibilityFilterNone & default 
		return EMeshVertexAttributePaintToolVisibilityType::None;
	}

	static FName GetVisibilityFilterNameFromEnum(EMeshVertexAttributePaintToolVisibilityType Enum)
	{
		switch (Enum)
		{
		case EMeshVertexAttributePaintToolVisibilityType::Unoccluded: 
			return VisibilityFilterUnoccluded;
		case EMeshVertexAttributePaintToolVisibilityType::None:
		default:
			return VisibilityFilterNone;
		}
	}

	static TSharedRef<SToolTip> MakeTooltipWithShortcut(const FText& TooltipText, const FInputChord& InputChord)
	{
		//return FText::Format(LOCTEXT("TooltipWithShorcut", "{0} {1}"), { TooltipText, InputChord.GetInputText() });

		//FFormatNamedArguments Args;
		//Args.Add(TEXT("ToolTipDescription"), TooltipText);
		//Args.Add(TEXT("Keybinding"), InputChord.GetInputText());
		//return FText::Format(NSLOCTEXT("ToolBar", "ToolTip + Keybinding", "{ToolTipDescription} ({Keybinding})"), Args);

		return IDocumentation::Get()->CreateToolTip(TooltipText, nullptr, FString(), FString(), InputChord.GetInputText());
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////
	FVertexAttributePaintToolDetailCustomization::FVertexAttributePaintToolDetailCustomization()
	{
		BrushQueryTypeNames.Add(QueryTypeInterpolated);
		BrushQueryTypeNames.Add(QueryTypeVertexFast);
		BrushQueryTypeNames.Add(QueryTypeVertexAccurate);

		BrushQueryTypeEnumFromNames.Add(QueryTypeInterpolated, EMeshVertexAttributePaintToolValueQueryType::Interpolated);
		BrushQueryTypeEnumFromNames.Add(QueryTypeVertexFast, EMeshVertexAttributePaintToolValueQueryType::NearestVertexFast);
		BrushQueryTypeEnumFromNames.Add(QueryTypeVertexAccurate, EMeshVertexAttributePaintToolValueQueryType::NearestVertexAccurate);

		BrushQueryTypeNameFromEnum.Add(EMeshVertexAttributePaintToolValueQueryType::Interpolated, QueryTypeInterpolated);
		BrushQueryTypeNameFromEnum.Add(EMeshVertexAttributePaintToolValueQueryType::NearestVertexFast, QueryTypeVertexFast);
		BrushQueryTypeNameFromEnum.Add(EMeshVertexAttributePaintToolValueQueryType::NearestVertexAccurate, QueryTypeVertexAccurate);

		VisibilityFilterNames.Add(VisibilityFilterNone);
		VisibilityFilterNames.Add(VisibilityFilterUnoccluded);

	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedRef<IDetailCustomization> FVertexAttributePaintToolDetailCustomization::MakeInstance()
	{
		return MakeShareable(new FVertexAttributePaintToolDetailCustomization);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	FVertexAttributePaintToolDetailCustomization::~FVertexAttributePaintToolDetailCustomization()
	{
		if (Tool.IsValid())
		{
			// TODO(ccaillaud) ==>>  Tool->OnSelectionChanged.RemoveAll(this);
		}

		Tool.Reset();
		ToolProperties.Reset();
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::OnSelectionChanged()
	{
		// TODO(ccaillaud) ==>>  ToolProperties.Get()->DirectEditState.Reset();
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::AddEditingModeRow(IDetailCategoryBuilder& EditModeCategory) const
	{
		// Add segmented control toggle for editing modes ("Brush" or "Selection")
		EditModeCategory.AddCustomRow(LOCTEXT("EditModeCategory", "Value Editing Mode"), false)
		.WholeRowContent()
		[
			SNew(SBox)
			.Padding(2.0f)
			[
				SNew(SSegmentedControl<EMeshVertexAttributePaintToolEditMode>)
				.Value_Lambda([this]()
					{
						return ToolProperties.IsValid() ? ToolProperties->EditingMode : EMeshVertexAttributePaintToolEditMode::Brush;
					})
				.OnValueChanged_Lambda([this](EMeshVertexAttributePaintToolEditMode Mode)
					{
						if (Tool.IsValid())
						{
							Tool->SetEditingMode(Mode);
							if (CurrentDetailBuilder)
							{
								CurrentDetailBuilder->ForceRefreshDetails();
							}
						}
					})
				+ SSegmentedControl<EMeshVertexAttributePaintToolEditMode>::Slot(EMeshVertexAttributePaintToolEditMode::Brush)
				.Text(LOCTEXT("BrushEditMode", "Brush"))
				.ToolTip(LOCTEXT("BrushEditModeTooltip", "Brush: Edit values by painting on mesh."))
				+ SSegmentedControl<EMeshVertexAttributePaintToolEditMode>::Slot(EMeshVertexAttributePaintToolEditMode::Mesh)
				.Text(LOCTEXT("MeshEditMode", "Mesh"))
				.ToolTip(LOCTEXT("MeshEditModeTooltip", "Mesh: Select vertices/edges/faces to edit values directly."))
			]
		];
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::AddColorModeRow(IDetailCategoryBuilder& MeshDisplayCategory) const
	{
		MeshDisplayCategory.AddCustomRow(LOCTEXT("ColorModeCategory", "Color Mode"), false)
		.WholeRowContent()
		[
			SNew(SBox)
			.Padding(2.0f)
			.HAlign(HAlign_Center)
			[
				SNew(SSegmentedControl<EMeshVertexAttributePaintToolColorMode>)
				.Value_Lambda([this]()
					{
						return ToolProperties.IsValid() ? ToolProperties->DisplayProperties.ColorMode : EMeshVertexAttributePaintToolColorMode::Greyscale;
					})
				.OnValueChanged_Lambda([this](EMeshVertexAttributePaintToolColorMode Mode)
					{
						if (Tool.IsValid())
						{
							Tool->SetColorMode(Mode);
						}
					})
				+ SSegmentedControl<EMeshVertexAttributePaintToolColorMode>::Slot(EMeshVertexAttributePaintToolColorMode::Greyscale)
				.Text(LOCTEXT("GreyscaleMode", "Greyscale"))
				.ToolTipWidget(MakeTooltipWithShortcut(LOCTEXT("GreyscaleModeTooltip", "Displays values by blending from black (0) to white (1)."), UMeshVertexAttributePaintToolBase::GreyscaleDisplayInputChord))
				+ SSegmentedControl<EMeshVertexAttributePaintToolColorMode>::Slot(EMeshVertexAttributePaintToolColorMode::Ramp)
				.Text(LOCTEXT("RampMode", "Ramp"))
				.ToolTipWidget(MakeTooltipWithShortcut(LOCTEXT("RampModeModeTooltip", "Values are mapped to the custom color ramp with 0 being the left most color of the ramp and 1 the right most color."), UMeshVertexAttributePaintToolBase::ColorMapDisplayInputChord))
				+ SSegmentedControl<EMeshVertexAttributePaintToolColorMode>::Slot(EMeshVertexAttributePaintToolColorMode::FullMaterial)
				.Text(LOCTEXT("MaterialMode", "Full Material"))
				.ToolTipWidget(MakeTooltipWithShortcut(LOCTEXT("MaterialModeTooltip", "Displays normal mesh materials with textures."), UMeshVertexAttributePaintToolBase::MaterialColorDisplayInputChord))
			]
		];
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////


	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::BuildColorRampMenu() const
	{
		const bool bShouldCloseWindowAfterMenuSelection = true;
		const bool bCloseSelfOnly = true;
		const bool bSearchable = false;
		FMenuBuilder MenuBuilder(true, nullptr, nullptr, bCloseSelfOnly, &FCoreStyle::Get(), bSearchable);

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("ColorRampOperations", "Color Ramp"));
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ColorRamp_ResetToDefault_Label", "Reset to default"),
				LOCTEXT("ColorRamp_ResetToDefault_Tooltip", "Reset the ramp to its default"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([WeakToolProperties = ToolProperties]()
						{
							if (TStrongObjectPtr<UMeshVertexAttributePaintToolProperties> ToolProperties = WeakToolProperties.Pin())
							{
								ToolProperties->ResetColorRamp();
								ToolProperties->SaveConfig();

								FPropertyChangedEvent PropertyChangedEvent(FMeshVertexAttributePaintToolDisplayProperties::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolDisplayProperties, ColorRamp)));
								ToolProperties->PostEditChangeProperty(PropertyChangedEvent);
							}
						}),
					FCanExecuteAction()
				)
			);

			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ColorRamp_AddToPresets_Label", "Add to presets"),
				LOCTEXT("ColorRamp_AddToPresets_Tooltip", "Add current ramp to presets"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([WeakToolProperties = ToolProperties]()
						{
							if (TStrongObjectPtr<UMeshVertexAttributePaintToolProperties> ToolProperties = WeakToolProperties.Pin())
							{
								ToolProperties->DisplayProperties.ColorRampPresets.Add(ToolProperties->DisplayProperties.ColorRamp);
								ToolProperties->SaveConfig();

								FPropertyChangedEvent PropertyChangedEvent(FMeshVertexAttributePaintToolDisplayProperties::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolDisplayProperties, ColorRamp)));
								ToolProperties->PostEditChangeProperty(PropertyChangedEvent);
							}
						}),
					FCanExecuteAction()
				)
			);
		}
		MenuBuilder.EndSection();

		auto MakeColorRampPresetWidget =
			[WeakToolProperties = ToolProperties](int32 PresetIndex)
			{
				TSharedRef<SColorGradientEditor> GradientEditor = SNew(SColorGradientEditor)
					.ViewMinInput(0.0f)
					.ViewMaxInput(1.0f)
					.ClampStopsToViewRange(true)
					.DrawColorAndAlphaSeparate(false)
					.IsEditingEnabled(false)
					.HideStops(true)
					.DesiredSizeOverride(FVector2D{ 256, 32 })
					;

				if (TStrongObjectPtr<UMeshVertexAttributePaintToolProperties> ToolProperties = WeakToolProperties.Pin())
				{
					GradientEditor->SetCurveOwner(&ToolProperties->DisplayProperties.ColorRampPresets[PresetIndex]);
				}

				return SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnClicked_Lambda(
					[WeakToolProperties, PresetIndex]()
					{
						if (TStrongObjectPtr<UMeshVertexAttributePaintToolProperties> ToolProperties = WeakToolProperties.Pin())
						{
							if (ToolProperties->DisplayProperties.ColorRampPresets.IsValidIndex(PresetIndex))
							{
								ToolProperties->DisplayProperties.ColorRamp.SetFrom(ToolProperties->DisplayProperties.ColorRampPresets[PresetIndex]);
								ToolProperties->SaveConfig();

								FPropertyChangedEvent PropertyChangedEvent(FMeshVertexAttributePaintToolDisplayProperties::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolDisplayProperties, ColorRamp)));
								ToolProperties->PostEditChangeProperty(PropertyChangedEvent);

								FSlateApplication::Get().DismissAllMenus();
							}
						}
						return FReply::Handled();
					})
					[
						GradientEditor
					]
					;
			};

		auto MakeColorRampPresetButton =
			[WeakToolProperties = ToolProperties](int32 PresetIndex)
			{
				return SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ToolTipText(LOCTEXT("ColorRamp_PresetDelete_Tooltip", "Delete this preset from preset list."))
					.OnClicked_Lambda(
					[WeakToolProperties, PresetIndex]()
					{
						if (TStrongObjectPtr<UMeshVertexAttributePaintToolProperties> ToolProperties = WeakToolProperties.Pin())
						{
							if (ToolProperties->DisplayProperties.ColorRampPresets.IsValidIndex(PresetIndex))
							{
								ToolProperties->DisplayProperties.ColorRampPresets.RemoveAt(PresetIndex);
								FPropertyChangedEvent PropertyChangedEvent(FMeshVertexAttributePaintToolDisplayProperties::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolDisplayProperties, ColorRamp)));
								ToolProperties->PostEditChangeProperty(PropertyChangedEvent);
								ToolProperties->SaveConfig();

								FSlateApplication::Get().DismissAllMenus();
							}
						}
						return FReply::Handled();
					})
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete").GetIcon())
						//.DesiredSizeOverride(FVector2D(24.0, 24.f))
					]
					;
			};

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("ColorRampPreset", "Presets"));
		{
			const int32 NumPresets = ToolProperties->DisplayProperties.ColorRampPresets.Num();
			for (int32 PresetIndex = 0; PresetIndex < NumPresets; ++PresetIndex)
			{
				TSharedRef<SWidget> ColorRampPresetWidget = MakeColorRampPresetWidget(PresetIndex);
				TSharedRef<SWidget> ColorRampPresetButton = MakeColorRampPresetButton(PresetIndex);

				TSharedRef<SWidget> ColorRampPresetCombinedWidget =
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(2.f)
					.AutoWidth()
					[
						ColorRampPresetWidget
					]
					+ SHorizontalBox::Slot()
					.Padding(2.f)
					.AutoWidth()
					[
						ColorRampPresetButton
					];
	
				MenuBuilder.AddWidget(ColorRampPresetCombinedWidget, FText::GetEmpty());
			}
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	void FVertexAttributePaintToolDetailCustomization::AddColorRampRow(IDetailCategoryBuilder& EditModeCategory) const
	{
		auto MakeColorRampWidget =
			[](
				EMeshVertexAttributePaintToolColorMode ColorMode, 
				FLinearColorRamp* ColorRamp,
				TWeakObjectPtr<UMeshVertexAttributePaintToolProperties> ToolProperties
				)
			{
				TSharedRef<SColorGradientEditor> GradientEditor = SNew(SColorGradientEditor)
					.ViewMinInput(0.0f)
					.ViewMaxInput(1.0f)
					.ClampStopsToViewRange(true)
					.DrawColorAndAlphaSeparate(false)
					.IsEditingEnabled_Lambda([ToolProperties, ColorMode]()
						{
							// Only the ramp one is editable 
							return ToolProperties.IsValid() && (ToolProperties->DisplayProperties.ColorMode == EMeshVertexAttributePaintToolColorMode::Ramp);
						})
					.Visibility_Lambda([ToolProperties, ColorMode]()
						{
							return (ToolProperties.IsValid() && (ToolProperties->DisplayProperties.ColorMode == ColorMode))
								? EVisibility::Visible
								: EVisibility::Collapsed;
						})
					;
				if (ToolProperties.IsValid() && ColorRamp)
				{
					GradientEditor->SetCurveOwner(ColorRamp);
				}
				return GradientEditor;
			};

		auto MakeColorRampOptionMenuWidget = [this](TWeakObjectPtr<UMeshVertexAttributePaintToolProperties> WeakToolProperties)
		{
			return SNew(SComboButton)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButtonWithIcon"))
				.ForegroundColor(FSlateColor::UseStyle())
				.ToolTipText(LOCTEXT("ColorRampOptionMenuToolTip", "Color Ramp Options Menu"))
				.OnGetMenuContent(this, &FVertexAttributePaintToolDetailCustomization::BuildColorRampMenu)
				.Visibility_Lambda([WeakToolProperties]()
					{
						return (WeakToolProperties.IsValid() && (WeakToolProperties->DisplayProperties.ColorMode == EMeshVertexAttributePaintToolColorMode::Ramp))
							? EVisibility::Visible
							: EVisibility::Collapsed;
					});
		};

		if (ToolProperties.IsValid())
		{
			TSharedRef<SColorGradientEditor> GreyScaleRampWidget =
				MakeColorRampWidget(EMeshVertexAttributePaintToolColorMode::Greyscale, &ToolProperties->DisplayProperties.GreyScaleColorRamp, ToolProperties);

			TSharedRef<SColorGradientEditor> CustomRampWidget =
				MakeColorRampWidget(EMeshVertexAttributePaintToolColorMode::Ramp, &ToolProperties->DisplayProperties.ColorRamp, ToolProperties);

			TSharedRef<SComboButton> CustomRampWidgetOptionMenu = MakeColorRampOptionMenuWidget(ToolProperties);

			TSharedRef<SColorGradientEditor> MaterialRampWidget =
				MakeColorRampWidget(EMeshVertexAttributePaintToolColorMode::FullMaterial, &ToolProperties->DisplayProperties.WhiteColorRamp, ToolProperties);

			EditModeCategory.AddCustomRow(LOCTEXT("ColorRampCategory", "Color Ramp"), false)
			.WholeRowContent()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					GreyScaleRampWidget
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(2.f)
					.FillWidth(1.0f)
					[
						CustomRampWidget
					]
					+ SHorizontalBox::Slot()
					.Padding(2.f)
					.AutoWidth()
					[
						CustomRampWidgetOptionMenu
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					MaterialRampWidget
				]
			];
		}
	}

	void FVertexAttributePaintToolDetailCustomization::AddValuesDisplayRows(IDetailCategoryBuilder& MeshDisplayCategory) const
	{
		MeshDisplayCategory.AddCustomRow(LOCTEXT("ValuesDisplay_ShowValues_Row", "Values Display - Show Values"), false)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("ValuesDisplay_ShowValues_Label", "Show Values"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("ValuesDisplay_ShowValues_Tooltip", "When enable, display the values on each vertex using Min/Max remapping "))
			]
			.ValueContent()
			[
				SNew(SCheckBox)
					.IsChecked_Lambda([ToolProperties = ToolProperties]()
						{
							const bool bIsChecked = (ToolProperties.IsValid() && ToolProperties->DisplayProperties.bShowValues);
							return bIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
					.OnCheckStateChanged_Lambda([Tool = Tool, ToolProperties = ToolProperties](ECheckBoxState InCheckBoxState)
						{
							if (ToolProperties.IsValid())
							{
								ToolProperties->DisplayProperties.bShowValues = (InCheckBoxState == ECheckBoxState::Checked);
								if (Tool.IsValid())
								{
									Tool->SetFocusInViewport();
								}
							}
						})
			];

		MeshDisplayCategory.AddCustomRow(LOCTEXT("ValuesDisplay_OnlySelected_Row", "Values Display - Only Selected"), false)
			.Visibility(TAttribute<EVisibility>::CreateLambda(
				[Tool = Tool]() 
				{ 
					return (Tool.IsValid() && !Tool->IsInBrushMode()) ? EVisibility::Visible: EVisibility::Collapsed;
				}))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("ValuesDisplay_OnlySelected_Label", "Only Selected"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("ValuesDisplay_onlySelected_Tooltip", "In mesh selection mode, only show values for the corresponding selected vertices"))
			]
			.ValueContent()
			[
				SNew(SCheckBox)
					.IsChecked_Lambda([ToolProperties = ToolProperties]()
						{
							const bool bIsChecked = (ToolProperties.IsValid() && ToolProperties->DisplayProperties.bShowValuesOnlySelected);
							return bIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
					.OnCheckStateChanged_Lambda([Tool= Tool, ToolProperties = ToolProperties](ECheckBoxState InCheckBoxState)
						{
							if (ToolProperties.IsValid())
							{
								ToolProperties->DisplayProperties.bShowValuesOnlySelected = (InCheckBoxState == ECheckBoxState::Checked);
								if (Tool.IsValid())
								{
									Tool->SetFocusInViewport();
								}
							}
						})
			];

		MeshDisplayCategory.AddCustomRow(LOCTEXT("ValuesDisplay_MinValue_Row", "Values Display - Min Value"), false)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("ValuesDisplay_MinValue_Label", "Min Value"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("ValuesDisplay_MinValue_Tooltip", "Minimum value mapping to the zero value of the painted map"))
			]
			.ValueContent()
			[
				SNew(SNumericEntryBox<float>)
					.IsEnabled_Lambda([ToolProperties = ToolProperties]()
						{
							return (ToolProperties.IsValid() && ToolProperties->DisplayProperties.bShowValues);
						})
					.Value_Lambda([ToolProperties = ToolProperties]()
						{
							const float MinValue = ToolProperties.IsValid() ? ToolProperties->DisplayProperties.MinValue : 0;
							return MinValue;
						})
					.OnValueChanged_Lambda([ToolProperties = ToolProperties](float Value)
						{
							if (ToolProperties.IsValid())
							{
								ToolProperties->DisplayProperties.MinValue = Value;
							}
						})
			];

		MeshDisplayCategory.AddCustomRow(LOCTEXT("ValuesDisplay_MaxValue_Row", "Values Display - Max Value"), false)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("ValuesDisplay_MaxValue_Label", "Max Value"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("ValuesDisplay_MaxValue_Tooltip", "Maximum value mapping to the one value of the painted map"))
			]
			.ValueContent()
			[
				SNew(SNumericEntryBox<float>)
					.IsEnabled_Lambda([ToolProperties = ToolProperties]()
						{
							return (ToolProperties.IsValid() && ToolProperties->DisplayProperties.bShowValues);
						})
					.Value_Lambda([ToolProperties = ToolProperties]()
						{
							const float MaxValue = ToolProperties.IsValid() ? ToolProperties->DisplayProperties.MaxValue : 0;
							return MaxValue;
						})
					.OnValueChanged_Lambda([ToolProperties = ToolProperties](float Value)
						{
							if (ToolProperties.IsValid())
							{
								ToolProperties->DisplayProperties.MaxValue = Value;
							}
						})
			];
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		CurrentDetailBuilder = &DetailBuilder;
	
		TArray<TWeakObjectPtr<UObject>> DetailObjects;
		DetailBuilder.GetObjectsBeingCustomized(DetailObjects);

		// Should be impossible to get multiple settings objects for a single tool
		ensure(DetailObjects.Num()==1);
		ToolProperties = Cast<UMeshVertexAttributePaintToolProperties>(DetailObjects[0]);
		Tool = Cast<UMeshVertexAttributePaintToolBase>(ToolProperties->GetOuter());
		// TODO(ccaillaud) ==>> Tool->OnSelectionChanged.AddSP(this, &FVertexAttributePaintToolDetailCustomization::OnSelectionChanged);
	
		// Editing Mode category 
		IDetailCategoryBuilder& EditModeCategory = DetailBuilder.EditCategory("Value Editing Mode", FText::GetEmpty(), ECategoryPriority::Important);
		{
			// Editing mode row [ Brush | Mesh ]
			AddEditingModeRow(EditModeCategory);

			switch (ToolProperties->EditingMode)
			{
			case EMeshVertexAttributePaintToolEditMode::Brush:
				AddBrushUI(DetailBuilder);
				AddBrushMirrorUI(DetailBuilder);
				break;
			case EMeshVertexAttributePaintToolEditMode::Mesh:
				AddSelectionUI(DetailBuilder);
				break;
			}
		}

		// Value Query category
		IDetailCategoryBuilder& QueryCategory = DetailBuilder.EditCategory("Query", FText::GetEmpty(), ECategoryPriority::Important);
		QueryCategory.InitiallyCollapsed(false);
		{
			// Value at brush
			AddBrushValueAtBrushRow(QueryCategory);

			// Value at brush query method
			AddBrushValueQueryMethodRow(QueryCategory);
		}

		IDetailCategoryBuilder& FiltersCategory = DetailBuilder.EditCategory("Filters", FText::GetEmpty(), ECategoryPriority::Important);
		FiltersCategory.InitiallyCollapsed(false);
		{
			AddVisibilityFilterRow(FiltersCategory);
		}

		// Mesh Display category
		IDetailCategoryBuilder& MeshDisplayCategory = DetailBuilder.EditCategory("MeshDisplay", FText::GetEmpty(), ECategoryPriority::Important);
		MeshDisplayCategory.InitiallyCollapsed(false);
		{
			AddColorModeRow(MeshDisplayCategory);

			AddColorRampRow(MeshDisplayCategory);

			AddValuesDisplayRows(MeshDisplayCategory);
		}

		// Hide all customized properties 
		HideProperty(DetailBuilder, GET_MEMBER_NAME_CHECKED(UMeshVertexAttributePaintToolProperties, EditingMode));
		HideProperty(DetailBuilder, GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolBrushProperties, BrushMode));
		HideProperty(DetailBuilder, GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolBrushProperties, BrushAreaMode));
		HideProperty(DetailBuilder, GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolBrushProperties, ValueAtBrush));
		HideProperty(DetailBuilder, GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolBrushProperties, AttributeValue));
		HideProperty(DetailBuilder, GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolDisplayProperties, ColorMode));

		HideProperty(DetailBuilder, GET_MEMBER_NAME_CHECKED(UMeshElementsVisualizerProperties, bVisible));
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::AddBrushModeRow(IDetailCategoryBuilder& BrushCategory) const
	{
		// add segmented control toggle for brush behavior modes ("Add", "Replace", etc..)
		BrushCategory.AddCustomRow(LOCTEXT("BrushModeCategory", "Brush Mode"), false)
		.WholeRowContent()
		[
			SNew(SBox)
			.Padding(2.0f)
			[
				SNew(SSegmentedControl<EMeshVertexAttributePaintToolEditOperation>)
				.Value_Lambda([this]()
					{
						return ToolProperties.IsValid() ? ToolProperties->BrushProperties.BrushMode : EMeshVertexAttributePaintToolEditOperation::Add;
					})
				.OnValueChanged_Lambda([this](EMeshVertexAttributePaintToolEditOperation Mode)
					{
						if (Tool.IsValid())
						{
							Tool->SetBrushMode(Mode);
						}
					})
				+ SSegmentedControl<EMeshVertexAttributePaintToolEditOperation>::Slot(EMeshVertexAttributePaintToolEditOperation::Add)
				.Text(LOCTEXT("BrushAddMode", "Add"))
				.ToolTip(LOCTEXT("BrushAddTooltip", "Add: Increases the existing vertex value by the brush value."))
				+ SSegmentedControl<EMeshVertexAttributePaintToolEditOperation>::Slot(EMeshVertexAttributePaintToolEditOperation::Replace)
				.Text(LOCTEXT("BrushReplaceMode", "Replace"))
				.ToolTip(LOCTEXT("BrushReplaceTooltip", "Replace: Overwrites the existing vertex value with the brush value."))
				+ SSegmentedControl<EMeshVertexAttributePaintToolEditOperation>::Slot(EMeshVertexAttributePaintToolEditOperation::Multiply)
				.Text(LOCTEXT("BrushMultiplyMode", "Multiply"))
				.ToolTip(LOCTEXT("BrushMultiplyTooltip", "Multiply: Scales the existing vertex value by the brush value."))
				+ SSegmentedControl<EMeshVertexAttributePaintToolEditOperation>::Slot(EMeshVertexAttributePaintToolEditOperation::Relax)
				.Text(LOCTEXT("BrushRelaxMode", "Relax"))
				.ToolTip(LOCTEXT("BrushRelaxTooltip", "Relax: Smooth the vertex value towards the average of its edge-connected neighbors."))
			]
		];
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::AddBrushFalloffModeRow(IDetailCategoryBuilder& BrushCategory) const
	{
		// add segmented control toggle for brush falloff modes ("Surface" or "Volume")
		BrushCategory.AddCustomRow(LOCTEXT("BrushFalloffModeCategory", "Brush Falloff Mode"), false)
		.WholeRowContent()
		[
			SNew(SBox)
			.Padding(2.0f)
			[
				SNew(SSegmentedControl<EMeshVertexPaintBrushAreaType>)
				.Value_Lambda([this]()
					{
						return ToolProperties.IsValid() ? ToolProperties->BrushProperties.BrushAreaMode : EMeshVertexPaintBrushAreaType::Connected;
					})
				.OnValueChanged_Lambda([this](EMeshVertexPaintBrushAreaType Mode)
					{
						if (Tool.IsValid())
						{
							Tool->SetBrushAreaMode(Mode);
						}
					})
				+ SSegmentedControl<EMeshVertexPaintBrushAreaType>::Slot(EMeshVertexPaintBrushAreaType::Connected)
				.Text(LOCTEXT("SurfaceMode", "Surface"))
				.ToolTip(LOCTEXT("SurfaceModeTooltip", "Surface: Falloff is based on the distance along the surface from the brush center to nearby connected vertices."))
				+ SSegmentedControl<EMeshVertexPaintBrushAreaType>::Slot(EMeshVertexPaintBrushAreaType::Volumetric)
				.Text(LOCTEXT("VolumeMode", "Volume"))
				.ToolTip(LOCTEXT("VolumeModeTooltip", "Volume: Falloff is based on the straight-line distance from the brush center to surrounding vertices"))
			]
		];
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::AddBrushRadiusRow(IDetailCategoryBuilder& BrushCategory) const
	{
		constexpr float MinBrushRadius = 0.01f;
		constexpr float MaxBrushRadius = 20.f;
		constexpr float DefaultBrushRadius = 10.0f;

		BrushCategory.AddCustomRow(LOCTEXT("BrushSizeCategory", "Brush Radius"), false)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BrushRadiusLabel", "Radius"))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.ToolTipText(LOCTEXT("BrushRadiusTooltip", "The radius of the brush in scene units."))
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.f)
			.FillWidth(1.0f)
			[
				SNew(SSpinBox<float>)
				.MinValue_Lambda([this]() { return Tool.IsValid() ? (Tool->GetBrushMinRadius()) : MinBrushRadius; })
				.MaxSliderValue_Lambda([this]() { return Tool.IsValid() ? (Tool->GetBrushMaxRadius()) : MaxBrushRadius; })
				.SupportDynamicSliderMaxValue(true)
				.Value_Lambda([this]() { return (Tool.IsValid()) ? Tool->GetCurrentBrushRadius() : MaxBrushRadius; })
				.OnValueChanged_Lambda([this](float NewValue)
					{
						if (Tool.IsValid())
						{
							Tool->SetBrushRadius(NewValue);
						}
					})
				.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
					{
						// Save radius so the next brush mode picks it up
						if (ToolProperties.IsValid())
						{
							ToolProperties->SaveConfig();
						}
						if (Tool.IsValid())
						{
							Tool->SetFocusInViewport();
						}
					})
			]
			+ SHorizontalBox::Slot()
			.Padding(2.f)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.IsChecked_Lambda([ToolProperties = ToolProperties]() -> ECheckBoxState
					{
						const bool bSync = ToolProperties.IsValid() && ToolProperties->bSyncBrushRadiusAcrossModes;
						return bSync ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([Tool = Tool, ToolProperties = ToolProperties](const ECheckBoxState NewState)
					{
						if (!ToolProperties.IsValid()) return;
						const bool bNewSync = (NewState == ECheckBoxState::Checked);
						ToolProperties->bSyncBrushRadiusAcrossModes = bNewSync;
						// Snap-on-enable: when re-linking, push the current size to every per-mode config
						// so toggling on never silently drops the active value.
						if (bNewSync && Tool.IsValid())
						{
							ToolProperties->SetSharedBrushSize(Tool->GetBrushAdaptiveSize());
						}
						ToolProperties->SaveConfig();
						if (Tool.IsValid())
						{
							Tool->SetFocusInViewport();
						}
					})
				.Padding(2.0f)
				.ToolTipText(LOCTEXT("SyncBrushRadiusTooltip",
					"When linked, the brush radius is shared across all brush modes. "
					"When unlinked, each mode remembers its own radius."))
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image_Lambda([ToolProperties = ToolProperties]()
						{
							const bool bSync = ToolProperties.IsValid() && ToolProperties->bSyncBrushRadiusAcrossModes;
							return FAppStyle::GetBrush(bSync ? "Icons.Link" : "Icons.Unlink");
						})
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::AddBrushValueRow(IDetailCategoryBuilder& BrushCategory) const
	{
		// All brushes has the same default range, use Ctrl to invert the brush for full range
		auto GetMinValue = [this]() -> float
			{
				return 0.0f;
			};

		auto GetMaxValue = [this]() -> float
			{
				return 1.0f;
			};


		BrushCategory.AddCustomRow(LOCTEXT("ValueCategory", "Value"), false)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ValueLabel", "Value"))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.ToolTipText(LOCTEXT("ValueTooltip", "The value of the attribute to be applied. Exact effect depends on the selected brush mode."))
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.f)
			.FillWidth(1.0f)
			[
				SNew(SSpinBox<float>)
				.MinValue_Lambda(GetMinValue)
				.MaxValue_Lambda(GetMaxValue)
				.Value(1.0f)
				.SupportDynamicSliderMaxValue(false)
				.Value_Lambda([this]()
					{
						return ToolProperties.IsValid() ? ToolProperties->BrushProperties.AttributeValue : 1.f;
					})
				.OnValueChanged_Lambda([this, GetMinValue, GetMaxValue](float NewValue)
					{
						if (ToolProperties.IsValid())
						{
							ToolProperties->BrushProperties.AttributeValue = FMath::Clamp(NewValue, GetMinValue(), GetMaxValue());
							FPropertyChangedEvent PropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolBrushProperties, AttributeValue)));
							ToolProperties->PostEditChangeProperty(PropertyChangedEvent);
						}
					})
				.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
					{
						if (Tool.IsValid())
						{
							Tool->SetFocusInViewport();
						}
					})
			]
			+ SHorizontalBox::Slot()
			.Padding(2.f)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.Visibility(TAttribute<EVisibility>::CreateLambda([ToolProperties = ToolProperties]()
					{
						const bool bIsRelax = ToolProperties.IsValid()
							&& ToolProperties->BrushProperties.BrushMode == EMeshVertexAttributePaintToolEditOperation::Relax;
						return bIsRelax ? EVisibility::Collapsed : EVisibility::Visible;
					}))
				.IsChecked_Lambda([Tool = Tool]()-> ECheckBoxState
					{
						const bool bPickerEnabled = (Tool.IsValid() && Tool->IsValuePickerEnabled());
						return bPickerEnabled? ECheckBoxState::Checked: ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([Tool = Tool](const ECheckBoxState NewState)
					{
						if (Tool.IsValid())
						{
							Tool->EnableValuePicker(NewState == ECheckBoxState::Checked);
						}
					})
				.Padding(2.0f)
				.ToolTip(MakeTooltipWithShortcut(LOCTEXT("ValuePickerTooltip", "Toggle for value picker"), UMeshVertexAttributePaintToolBase::ToggleValuePickerInputChord))
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.EyeDropper"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::AddBrushFalloffRow(IDetailCategoryBuilder& BrushCategory) const
	{
		BrushCategory.AddCustomRow(LOCTEXT("BrushFalloffCategory", "Brush Falloff"), false)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("BrushFalloffLabel", "Falloff"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("BrushFalloffTooltip", "Amount of falloff in the [0,1] range"))
			]
			.ValueContent()
			[
				SNew(SSpinBox<float>)
					.MinValue(0)
					.MaxValue(1.0f)
					.MaxSliderValue(1.f)
					.Value(1.0f)
					.SupportDynamicSliderMaxValue(false)
					.Value_Lambda([this]() { return (Tool.IsValid()) ? Tool->GetBrushFalloff() : 1.0f; })
					.OnValueChanged_Lambda([this](float NewValue)
						{
							if (Tool.IsValid())
							{
								Tool->SetBrushFalloff(NewValue);
							}
						})
					.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
						{
							if (Tool.IsValid())
							{
								Tool->SetFocusInViewport();
							}
						})
			];
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::AddBrushAccumulateRow(IDetailCategoryBuilder& BrushCategory) const
	{
		// Visible only when brush mode = Relax — checkbox is bound to RelaxBrushAdvancedConfig.bAccumulate.
		BrushCategory.AddCustomRow(LOCTEXT("BrushAccumulateCategory", "Brush Accumulate"), false)
			.Visibility(TAttribute<EVisibility>::CreateLambda([ToolProperties = ToolProperties]()
			{
				const bool bIsRelax = ToolProperties.IsValid()
					&& ToolProperties->BrushProperties.BrushMode == EMeshVertexAttributePaintToolEditOperation::Relax;
				return bIsRelax ? EVisibility::Visible : EVisibility::Collapsed;
			}))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("BrushAccumulateLabel", "Accumulate"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("BrushAccumulateTooltip",
						"When enabled, dragging or dwelling the relax brush keeps smoothing further. "
						"When disabled, each stroke produces a single bounded smoothing step."))
			]
			.ValueContent()
			[
				SNew(SCheckBox)
					.IsChecked_Lambda([ToolProperties = ToolProperties]()
					{
						const bool bChecked = ToolProperties.IsValid() && ToolProperties->RelaxBrushAdvancedConfig.bAccumulate;
						return bChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([ToolProperties = ToolProperties](ECheckBoxState NewState)
					{
						if (ToolProperties.IsValid())
						{
							ToolProperties->RelaxBrushAdvancedConfig.bAccumulate = (NewState == ECheckBoxState::Checked);
							ToolProperties->SaveConfig();
						}
					})
			];
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::AddBrushValueAtBrushRow(IDetailCategoryBuilder& BrushCategory) const
	{
		BrushCategory.AddCustomRow(LOCTEXT("ValueAtBrushCategory", "ValueAtBrush"), false)
			.NameContent()
			[
				SNew(STextBlock)
					.Text_Lambda([Tool = Tool]()
						{
							return (Tool.IsValid() && Tool->IsInBrushMode())
								? LOCTEXT("ValueAtBrushLabel", "Value At Brush")
								: LOCTEXT("ValueOfSelectionLabel", "Value Of Selection");
						})
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText_Lambda([Tool = Tool]()
						{
							return (Tool.IsValid() && Tool->IsInBrushMode())
								? LOCTEXT("ValueAtBrushTooltip", "The value where the brush is loclated. The exact method used to compute it can be changed in the combobox below")
								: LOCTEXT("ValueOfSelectionTooltip", "The average value of selected vertices.");
						})
			]
			.ValueContent()
			[
				SNew(STextBlock)
					.Justification(ETextJustify::Left)
					.IsEnabled(false)
					.Text_Lambda([this]()
						{
							return FText::AsNumber(ToolProperties.IsValid() ? ToolProperties->BrushProperties.ValueAtBrush: 1.f);
						})
			];
	}

	void FVertexAttributePaintToolDetailCustomization::AddBrushValueQueryMethodRow(IDetailCategoryBuilder& BrushCategory) const
	{
		BrushCategory.AddCustomRow(LOCTEXT("ValueQueryMethodBrushCategory", "ValueAtBrush"), false)
			.Visibility(TAttribute<EVisibility>::CreateLambda(
				[Tool = Tool]() 
				{ 
					return (Tool.IsValid() && Tool->IsInBrushMode()) ? EVisibility::Visible : EVisibility::Collapsed;
				}))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("ValueQueryMethodBrushLabel", "Value At Brush Query Type"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("ValueQueryMethodBrushTooltip", "The method used to query the value under the brush"))
			]
			.ValueContent()
			[
				SNew(SComboBox<FName>)
					.ComboBoxStyle(&FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>(TEXT("ComboBox")))
					.OptionsSource(&BrushQueryTypeNames)
					.HasDownArrow(true)
					.InitiallySelectedItem(QueryTypeVertexFast)
					.ContentPadding(0.f)
					.OnGenerateWidget_Lambda([this](FName Item)
						{
							return SNew(STextBlock)
								.Margin(FMargin(2,1))
								.Justification(ETextJustify::Left)
								.Text(FText::FromString(Item.ToString()));
						})
					.OnSelectionChanged_Lambda([this](const FName InSelection, ESelectInfo::Type InSelectInfo)
						{
							if (ToolProperties.IsValid())
							{
								ToolProperties->BrushProperties.ValueQueryType = BrushQueryTypeEnumFromNames.FindRef(InSelection, EMeshVertexAttributePaintToolValueQueryType::NearestVertexFast);
							}
						})
					.Content()
					[
						SNew(STextBlock)
							.Margin(FMargin(2, 1))
							.Text_Lambda([this]()
								{
									const FName CurrentItemName = ToolProperties.IsValid()
										? BrushQueryTypeNameFromEnum.FindRef(ToolProperties->BrushProperties.ValueQueryType, QueryTypeVertexFast)
										: QueryTypeVertexFast;
									return FText::FromName(CurrentItemName);
								})
					]
			];
	}

	void FVertexAttributePaintToolDetailCustomization::AddVisibilityFilterRow(IDetailCategoryBuilder& BrushCategory) const
	{
		const EMeshVertexAttributePaintToolVisibilityType InitialValue = (ToolProperties.IsValid()) ? ToolProperties->BrushProperties.VisibilityFilter :EMeshVertexAttributePaintToolVisibilityType::None;
			

		BrushCategory.AddCustomRow(LOCTEXT("VisibilityFilterCategory", "VisibilityFilter"), false)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("VisibilityFilterLabel", "Visibility Filter"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("VisibilityFilterTooltip", "Method to use to filter faces to paint on"))
			]
			.ValueContent()
			[
				SNew(SComboBox<FName>)
					.ComboBoxStyle(&FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>(TEXT("ComboBox")))
					.OptionsSource(&VisibilityFilterNames)
					.HasDownArrow(true)
					.InitiallySelectedItem(GetVisibilityFilterNameFromEnum(InitialValue))
					.ContentPadding(0.f)
					.OnGenerateWidget_Lambda([this](FName Item)
						{
							return SNew(STextBlock)
								.Margin(FMargin(2, 1))
								.Justification(ETextJustify::Left)
								.Text(FText::FromString(Item.ToString()));
						})
					.OnSelectionChanged_Lambda([this](const FName InSelection, ESelectInfo::Type InSelectInfo)
						{
							if (ToolProperties.IsValid())
							{
								ToolProperties->BrushProperties.VisibilityFilter = GetVisibilityFilterEnumFromName(InSelection);
							}
						})
					.Content()
					[
						SNew(STextBlock)
							.Margin(FMargin(2, 1))
							.Text_Lambda([this]()
								{
									const EMeshVertexAttributePaintToolVisibilityType CurrentValue = (ToolProperties.IsValid()) ? ToolProperties->BrushProperties.VisibilityFilter : EMeshVertexAttributePaintToolVisibilityType::None;
									return FText::FromName(GetVisibilityFilterNameFromEnum(CurrentValue));
								})
					]
			];
	}

	void FVertexAttributePaintToolDetailCustomization::AddBrushHitBackFacesRow(IDetailCategoryBuilder& BrushCategory) const
	{
		BrushCategory.AddCustomRow(LOCTEXT("BrushHitBackFacesCategory", "Brush Hit Back Faces"), false)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("BrushHitBackFacesLabel", "Hit Back faces"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("BrushHitBackFacesTooltip", "Allow the brush to hit the back side of the mesh"))
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]()
					{
						const bool bIsChecked = (Tool.IsValid() && Tool->BrushProperties && Tool->BrushProperties->bHitBackFaces);
						return bIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
					{
						if (Tool.IsValid() && Tool->BrushProperties)
						{
							Tool->BrushProperties->bHitBackFaces = (InCheckBoxState == ECheckBoxState::Checked);
							Tool->SetFocusInViewport();
						}
					})
			];
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::AddBrushUI(IDetailLayoutBuilder& DetailBuilder) const
	{
		IDetailCategoryBuilder& BrushCategory = DetailBuilder.EditCategory("Brush", FText::GetEmpty(), ECategoryPriority::Important);
		{
			// Brush modes [ Add | Replace | Multiply | Relax ]
			AddBrushModeRow(BrushCategory);

			// Brush Falloff modes [ Surface | Volume ]
			AddBrushFalloffModeRow(BrushCategory);

			// Brush radius field
			AddBrushRadiusRow(BrushCategory);

			// Brush value field
			AddBrushValueRow(BrushCategory);

			// Brush value field
			AddBrushFalloffRow(BrushCategory);

			// Relax-only: Accumulate toggle
			AddBrushAccumulateRow(BrushCategory);

			// Brush backface hit option
			AddBrushHitBackFacesRow(BrushCategory);
		}
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::MakeBrushMirrorEnableWidget() const
	{
		return SNew(SCheckBox)
			.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
			.IsChecked_Lambda([Tool = Tool]()-> ECheckBoxState
				{
					const bool bIsBrushMirroringEnabled = (Tool.IsValid() && Tool->IsBrushMirroringEnabled());
					return bIsBrushMirroringEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			.OnCheckStateChanged_Lambda([Tool = Tool](const ECheckBoxState NewState)
				{
					if (Tool.IsValid())
					{
						Tool->EnableBrushMirroring(NewState == ECheckBoxState::Checked);
					}
				})
			.Padding(2.0f)
			.ToolTip(MakeTooltipWithShortcut(LOCTEXT("BrushMirrorEnabledTooltip", "Enable or disable brush mirrorring"), UMeshVertexAttributePaintToolBase::MirrorValuesInputChord))
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
					.Image(FAppStyle::GetBrush("UMGEditor.Mirror"))
					.ColorAndOpacity(FSlateColor::UseForeground())
			];
	}

	void FVertexAttributePaintToolDetailCustomization::AddBrushMirrorEnableRow(IDetailCategoryBuilder& BrushCategory) const
	{
		BrushCategory.AddCustomRow(LOCTEXT("BrushMirrorEnableCategory", "Brush Mirror Enable"), false)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("BrushMirrorEnableLabel", "Enable Mirroring"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("BrushMirrorEnableTooltip", "Enable/Disable brush mirroring"))
			]
			.ValueContent()
			[
				MakeBrushMirrorEnableWidget()
			];
	}

	void FVertexAttributePaintToolDetailCustomization::AddBrushMirrorPlaneRow(IDetailCategoryBuilder& BrushCategory) const
	{
		BrushCategory.AddCustomRow(LOCTEXT("BrushMirrorPlaneCategory", "Brush Mirror Axis"), false)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("BrushMirrorPlaneLabel", "Mirror Plane"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("BrushMirrorPlaneTooltip", "Plane axis to use for mirroring"))
			]
			.ValueContent()
			[
				MakeMirrorShowAndAxisWidget()
			];
	}

	void FVertexAttributePaintToolDetailCustomization::AddBrushMirrorHideOnBrushStrokeRow(IDetailCategoryBuilder& BrushCategory) const
	{
		BrushCategory.AddCustomRow(LOCTEXT("BrushMirrorHideOnBrushStrokeCategory", "Brush Mirror Hide On Brush Stroke"), false)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("BrushMirrorHideOnBrushStrokeLabel", "Hide Plane When Painting"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("BrushMirrorHideOnBrushStrokeTooltip", "When painting, hide the mirror plane widget in the viewport."))
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([ToolProperties = ToolProperties]()
					{
						const bool bIsChecked = (ToolProperties.IsValid() && ToolProperties->MirrorProperties.bHideOnBrushStroke);
						return bIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([Tool = Tool, ToolProperties = ToolProperties](ECheckBoxState InCheckBoxState)
					{
						if (ToolProperties.IsValid())
						{
							ToolProperties->MirrorProperties.bHideOnBrushStroke = (InCheckBoxState == ECheckBoxState::Checked);
							if (Tool.IsValid())
							{
								Tool->SetFocusInViewport();
							}
						}
					})
			];
	}

	void FVertexAttributePaintToolDetailCustomization::AddBrushMirrorObjectSpaceRow(IDetailCategoryBuilder& BrushCategory) const
	{
		BrushCategory.AddCustomRow(LOCTEXT("BrushMirrorObjectSpaceCategory", "Brush Mirror Object Space"), false)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("BrushMirrorObjectSpaceLabel", "Object Space"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("BrushMirrorObjectSpaceTooltip", "When checked, the mirror plane widget will be centered on the object bounds."))
			]
			.ValueContent()
			[
				MakeMirrorObjectSpaceCheckboxWidget()
			];
	}

	void FVertexAttributePaintToolDetailCustomization::AddBrushMirrorUI(IDetailLayoutBuilder& DetailBuilder) const
	{
		IDetailCategoryBuilder& BrushCategory = DetailBuilder.EditCategory("BrushMirror", FText::GetEmpty(), ECategoryPriority::Important);
		{
			AddBrushMirrorEnableRow(BrushCategory);

			AddBrushMirrorPlaneRow(BrushCategory);

			AddBrushMirrorHideOnBrushStrokeRow(BrushCategory);

			AddBrushMirrorObjectSpaceRow(BrushCategory);
		}
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::MakeDragToolToolbar() const
	{
		FSlimHorizontalToolBarBuilder ToolbarBuilder(MakeShared<FUICommandList>(), FMultiBoxCustomization::None);
		ToolbarBuilder.SetStyle(FModelingToolsEditorModeStyle::Get().Get(), "PolyEd.SelectionToolbar");
		ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

		ToolbarBuilder.BeginSection("SelectionDragTool");
		ToolbarBuilder.BeginBlockGroup();

		auto AddToggleButtonForBool = [&ToolbarBuilder, this](EMeshSelectorTool SelectionTool, const FText& Label, const FText& Tooltip, const FName IconName)
			{
				ToolbarBuilder.AddToolBarButton(FUIAction(
					FExecuteAction::CreateLambda([this, SelectionTool]()
						{
							if (Tool.IsValid())
							{
								Tool->SetMeshSelectionTool(SelectionTool);
							}
						}),
					FCanExecuteAction::CreateLambda([this]()
						{
							return ToolProperties.IsValid() ? ToolProperties->EditingMode == EMeshVertexAttributePaintToolEditMode::Mesh : false;
						}),
					FIsActionChecked::CreateLambda([this, SelectionTool]()
						{
							return Tool.IsValid() ? (Tool->GetMeshSelectionTool() == SelectionTool) : false;
						})),
					NAME_None,	// Extension hook
					Label,		// Label
					Tooltip,	// Tooltip
					FSlateIcon(FAppStyle::GetAppStyleSetName(), IconName),
					EUserInterfaceActionType::ToggleButton);
			};




		ToolbarBuilder.EndBlockGroup();
		ToolbarBuilder.EndSection();

		return ToolbarBuilder.MakeWidget();
	}

	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::MakeSelectionElementsToolbar() const
	{
		FSlimHorizontalToolBarBuilder ToolbarBuilder(MakeShared<FUICommandList>(), FMultiBoxCustomization::None);
		ToolbarBuilder.SetStyle(FModelingToolsEditorModeStyle::Get().Get(), "PolyEd.SelectionToolbar");
		ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

		ToolbarBuilder.BeginSection("SelectionFilter");
		ToolbarBuilder.BeginBlockGroup();

		auto AddToggleButtonSelectionTool = [&ToolbarBuilder, this](EMeshSelectorTool SelectionTool, const FText& Label, const FText& Tooltip, const FSlateIcon& Icon)
			{
				ToolbarBuilder.AddToolBarButton(FUIAction(
					FExecuteAction::CreateLambda([this, SelectionTool]()
						{
							if (Tool.IsValid())
							{
								Tool->SetMeshSelectionTool(SelectionTool);
							}
						}),
					FCanExecuteAction::CreateLambda([this]()
						{
							return ToolProperties.IsValid() ? ToolProperties->EditingMode == EMeshVertexAttributePaintToolEditMode::Mesh : false;
						}),
					FIsActionChecked::CreateLambda([this, SelectionTool]()
						{
							return Tool.IsValid() ? (Tool->GetMeshSelectionTool() == SelectionTool) : false;
						})),
					NAME_None,	// Extension hook
					Label,		// Label
					Tooltip,	// Tooltip
					Icon,
					EUserInterfaceActionType::ToggleButton);
			};

		AddToggleButtonSelectionTool(
			EMeshSelectorTool::Marquee,
			LOCTEXT("RectangleSelectionToolLabel", "Rectangle"),
			LOCTEXT("RectangleSelectionToolTooltip", "Use rectangle selection tool on click and drag."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.StrictBoxSelect")
		);
		AddToggleButtonSelectionTool(
			EMeshSelectorTool::Lasso,
			LOCTEXT("LassoSelectionToolLabel", "Lasso"),
			LOCTEXT("LassoSelectionToolTooltip", "Use lasso selection tool on click and drag."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.SetLassoSelect")
		);

		auto AddToggleButtonComponentMode = [&ToolbarBuilder, this](EComponentSelectionMode Mode, const FText& Label, const FText& Tooltip, const FSlateIcon& Icon)
			{
				ToolbarBuilder.AddToolBarButton(FUIAction(
					FExecuteAction::CreateLambda([this, Mode]()
						{
							if (Tool.IsValid())
							{
								Tool->SetComponentSelectionMode(Mode);
							}
						}),
					FCanExecuteAction::CreateLambda([this]()
						{
							return ToolProperties.IsValid() ? ToolProperties->EditingMode == EMeshVertexAttributePaintToolEditMode::Mesh : false;
						}),
					FIsActionChecked::CreateLambda([this, Mode]()
						{
							return ToolProperties.IsValid() ? (ToolProperties->SelectionProperties.ComponentSelectionMode == Mode) : false;
						})),
					NAME_None,	// Extension hook
					Label,		// Label
					Tooltip,	// Tooltip
					Icon,
					EUserInterfaceActionType::ToggleButton);
			};

		ToolbarBuilder.AddSeparator();
		AddToggleButtonComponentMode(
			EComponentSelectionMode::Vertices,
			LOCTEXT("VerticesLabel", "Vertices"),
			LOCTEXT("VerticesTooltip", "Select mesh vertices."),
			FSlateIcon(FModelingToolsEditorModeStyle::Get()->GetStyleSetName(), "PolyEd.SelectCorners")
		);
		AddToggleButtonComponentMode(
			EComponentSelectionMode::Edges,
			LOCTEXT("EdgesLabel", "Edges"),
			LOCTEXT("EdgesTooltip", "Select mesh edges."),
			FSlateIcon(FModelingToolsEditorModeStyle::Get()->GetStyleSetName(), "PolyEd.SelectEdges")
		);
		AddToggleButtonComponentMode(
			EComponentSelectionMode::Faces,
			LOCTEXT("FacesLabel", "Faces"),
			LOCTEXT("FacesTooltip", "Select mesh faces."),
			FSlateIcon(FModelingToolsEditorModeStyle::Get()->GetStyleSetName(), "PolyEd.SelectFaces")
		);

		ToolbarBuilder.EndBlockGroup();
		ToolbarBuilder.EndSection();

		return ToolbarBuilder.MakeWidget();
	}

	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::MakeSelectionIsolationWidget() const
	{
#if 0
		return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.HAlign(HAlign_Center)
		.ToolTipText(LOCTEXT("IsolateSelectedTooltip",
			"Shows only the selected faces in the viewport.\n"
			"Weight editing operations will not affect hidden vertices.\n"
			"NOTE: This only works on the target (main) mesh."))
		.IsEnabled_Lambda([this]()
			{
				if (Tool.IsValid())
				{
					// isolated selection only available on main mesh (for now)
					const bool bHasSelection = Tool->GetMainMeshSelector()->IsAnyComponentSelected();
					const bool bAlreadyIsolatingSelection = Tool->GetSelectionIsolator()->IsSelectionIsolated();
					return bHasSelection || bAlreadyIsolatingSelection;
				}
				return false;
			})
		.IsChecked_Lambda([this]()
			{
				if (Tool.IsValid())
				{
					const bool bAlreadyIsolatingSelection = Tool->GetSelectionIsolator()->IsSelectionIsolated();
					return bAlreadyIsolatingSelection ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}
				return ECheckBoxState::Unchecked;
			})
		.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
			{
				if (UWeightToolSelectionIsolator* SelectionIsolator = Tool.IsValid() ? Tool->GetSelectionIsolator() : nullptr)
				{
					if (InCheckBoxState == ECheckBoxState::Checked)
					{
						SelectionIsolator->IsolateSelectionAsTransaction();
					}
					else
					{
						SelectionIsolator->UnIsolateSelectionAsTransaction();
					}
				}
			})
		[
			SNew(STextBlock)
				.Text_Lambda([this]()
					{
						if (!Tool.IsValid() || Tool->GetSelectionIsolator()->IsSelectionIsolated())
						{
							return LOCTEXT("ShowAllButtonLabel", "Show Full Mesh");
						}

						return LOCTEXT("IsolateButtonLabel", "Isolate Selected");
					})
		];

#else //0
		return SNullWidget::NullWidget;

#endif //0
	}

	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::MakeSelectionEditActionsToolbar() const
	{
		auto MakeButton = [this](const FText& InText, const FText& InTooltip, const FInputChord& InChord, TFunction<void()> OnClicked)
			{
				return SNew(SButton)
					.IsEnabled_Lambda([this]
						{
							return Tool.IsValid() ? Tool->HasSelection() : false;
						})
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(InText)
					.ToolTip(MakeTooltipWithShortcut(InTooltip, InChord))
					.OnClicked_Lambda([OnClicked]()
						{
							OnClicked();
							return FReply::Handled();
						});
			};

		// [ GROW / SHRINK / FLOOD ]
		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(2.f, WeightEditVerticalPadding)
				[
					MakeButton(
						LOCTEXT("GrowSelectionButtonLabel", "Grow"),
						LOCTEXT("GrowSelectionTooltip", "Grow the current selection by adding connected neighbors to current selection."),
						UMeshVertexAttributePaintToolBase::GrowSelectionInputChord,
						[this]() { if (Tool.IsValid()) { Tool->GrowSelection(); } }
					)
				]
				+ SHorizontalBox::Slot()
				.Padding(2.f, WeightEditVerticalPadding)
				[
					MakeButton(
						LOCTEXT("ShrinkSelectionButtonLabel", "Shrink"),
						LOCTEXT("ShrinkSelectionTooltip", "Shrink the current selection by removing components on the border of the current selection."),
						UMeshVertexAttributePaintToolBase::ShrinkSelectionInputChord,
						[this]() { if (Tool.IsValid()) { Tool->ShrinkSelection(); } }
					)
				]
				+ SHorizontalBox::Slot()
					.Padding(2.f, WeightEditVerticalPadding)
					[
						MakeButton(
							LOCTEXT("InvertSelectionButtonLabel", "Invert"),
							LOCTEXT("InvertSelectionTooltip", "Invert the current selection."),
							FInputChord(),
							[this]() { if (Tool.IsValid()) { Tool->InvertSelection(); } }
						)
					]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(2.f, WeightEditVerticalPadding)
				[
					MakeButton(
						LOCTEXT("FloodSelectionButtonLabel", "Flood"),
						LOCTEXT("FloodSelectionTooltip", "Flood the current selection by adding all connected components to the current selection."),
						UMeshVertexAttributePaintToolBase::FloodSelectionInputChord,
						[this]() { if (Tool.IsValid()) { Tool->FloodSelection(); } }
					)
				]
				+ SHorizontalBox::Slot()
				.Padding(2.f, WeightEditVerticalPadding)
				[
					MakeButton(
						LOCTEXT("BorderSelectionButtonLabel", "Convert to Border"),
						LOCTEXT("BorderSelectionTooltip", "Select vertices on the border of the current selection."),
						FInputChord(),
						[this]() { if (Tool.IsValid()) { Tool->SelectBorder(); } }
					)
				]
			];
	}

	void FVertexAttributePaintToolDetailCustomization::AddSelectionElementsRow(IDetailCategoryBuilder& EditSelectionCategory) const
	{
		TSharedRef<SWidget> ElementsToolBar = MakeSelectionElementsToolbar();
		TSharedRef<SWidget> IsolationWidget = MakeSelectionIsolationWidget();
		TSharedRef<SWidget> SelectionEditActionsToolBar = MakeSelectionEditActionsToolbar();

		EditSelectionCategory.AddCustomRow(LOCTEXT("EditSelectionRow", "Edit Selection"), false)
		.WholeRowContent()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(2.0f, WeightEditVerticalPadding)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						ElementsToolBar
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						IsolationWidget
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SelectionEditActionsToolBar
			]
		];
	}

	void FVertexAttributePaintToolDetailCustomization::AddEmptySelectionWarningRow( IDetailCategoryBuilder& EditValuesCategory) const
	{
		EditValuesCategory.AddCustomRow(LOCTEXT("SelectMessageRow", "Select Vertices"), false)
		.WholeRowContent()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Visibility_Lambda([this]()
				{
					return Tool.IsValid() && !Tool->HasSelection() ? EVisibility::Visible : EVisibility::Collapsed;
				})
			.Text_Lambda([this]()
				{
					return LOCTEXT("NothingSelectedLabel", "Select vertices on target mesh to edit weights...");
				})
		];
	}

	void FVertexAttributePaintToolDetailCustomization::MakeSelectionOperationRow(
		IDetailCategoryBuilder& EditValuesCategory,
		const FText& RowName,
		const TSharedRef<SWidget>& ButtonWidget, 
		const TSharedRef<SWidget>& ValueWidget) const
	{
		EditValuesCategory.AddCustomRow(RowName, false)
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			.IsEnabled_Lambda([this]
				{
					return Tool.IsValid() ? Tool->HasSelection() : false;
				})
			+ SHorizontalBox::Slot()
			[
				SNew(SBox)[ButtonWidget]
			]
			+ SHorizontalBox::Slot()
			[
				ValueWidget
			]
		];
	}

	void FVertexAttributePaintToolDetailCustomization::MakeSelectionAddMultiplySliderOperationRow(IDetailCategoryBuilder& EditValuesCategory) const
	{
		struct FSliderState
		{
			float Value = 0.f;
		};

		TSharedPtr<FSliderState> SliderState = MakeShared<FSliderState>();

		auto GetCurrentValue = [SliderState]()
			{
				return SliderState->Value;
			};

		auto ResetValue = [SliderState](EMeshVertexAttributePaintToolEditOperation Mode)
			{
				switch (Mode)
				{
				case EMeshVertexAttributePaintToolEditOperation::Add:
					SliderState->Value = 0.f;
					break;
				case EMeshVertexAttributePaintToolEditOperation::Multiply:
					SliderState->Value = 1.f;
					break;
				}
			};

		TSharedRef<SSegmentedControl<EMeshVertexAttributePaintToolEditOperation>> AddMultiplyButtonsWidget =
			SNew(SSegmentedControl<EMeshVertexAttributePaintToolEditOperation>)
			.Value(EMeshVertexAttributePaintToolEditOperation::Add)
			.OnValueChanged_Lambda([ResetValue](EMeshVertexAttributePaintToolEditOperation Mode)
				{
					ResetValue(Mode);
				})
			+ SSegmentedControl<EMeshVertexAttributePaintToolEditOperation>::Slot(EMeshVertexAttributePaintToolEditOperation::Add)
			.Text(LOCTEXT("BrushAddMode", "Add"))
			.ToolTip(LOCTEXT("BrushAddModeTooltip", "Add: Applies interactively the current value plus the flood value to the new value when dragging the slider."))
			+ SSegmentedControl<EMeshVertexAttributePaintToolEditOperation>::Slot(EMeshVertexAttributePaintToolEditOperation::Multiply)
			.Text(LOCTEXT("BrushMultiplyMode", "Multiply"))
			.ToolTip(LOCTEXT("BrushMultiplyModeTooltip", "Multiply: Applies interactively the current value multiplied by the flood value to the new value when dragging the slider."))
			;

		auto GetMinValue = [AddMultiplyButtonsWidget]() -> float
			{
				switch (AddMultiplyButtonsWidget->GetValue())
				{
				case EMeshVertexAttributePaintToolEditOperation::Add: return -1.f;
				case EMeshVertexAttributePaintToolEditOperation::Multiply: return 0.f;
				}
				return 0.f;
			};

		auto GetMaxValue = [AddMultiplyButtonsWidget]() -> float
			{
				switch (AddMultiplyButtonsWidget->GetValue())
				{
				case EMeshVertexAttributePaintToolEditOperation::Add: return 1.f;
				case EMeshVertexAttributePaintToolEditOperation::Multiply: return 2.f;
				}
				return 0.f;
			};



		auto ApplyOperation = [this, SliderState, AddMultiplyButtonsWidget](float NewValue)
			{
				if (Tool.IsValid())
				{
					SliderState->Value = NewValue;
					// bWithTransaction = false because we already are tracking our own transaction ( see OnBeginSliderMovement_Lambda and OnEndSliderMovement_Lambda)
					Tool->ApplyValueToSelection(AddMultiplyButtonsWidget->GetValue(), NewValue, /*bWithTransaction*/false);
				}
			};

		TSharedRef<SWidget> SliderValueWidget = 
			SNew(SSpinBox<float>)
			.MinSliderValue_Lambda(GetMinValue)
			.MaxSliderValue_Lambda(GetMaxValue)
			.MinValue_Lambda(GetMinValue)
			.MaxValue_Lambda(GetMaxValue)
			.Value_Lambda(GetCurrentValue)
			.OnValueChanged_Lambda([ApplyOperation](float NewValue)
				{
					ApplyOperation(NewValue);
				})
			.OnValueCommitted_Lambda([ApplyOperation](float NewValue, ETextCommit::Type CommitType)
				{
					ApplyOperation(NewValue);
				})
			.OnBeginSliderMovement_Lambda([this]()
				{
					if (Tool.IsValid())
					{
						Tool->BeginChange();
					}
				})
			.OnEndSliderMovement_Lambda([this, ResetValue, AddMultiplyButtonsWidget](float)
				{
					if (Tool.IsValid())
					{
						Tool->EndChange();
						Tool->SetFocusInViewport();
					}
					ResetValue(AddMultiplyButtonsWidget->GetValue());
				})
			.ToolTipText(LOCTEXT("FloodWeightsToolTip", "Drag the slider to interactively adjust values on the selected vertices."))
			;

		MakeSelectionOperationRow(EditValuesCategory, LOCTEXT("FloodValuesRow", "Flood Values Slider"), AddMultiplyButtonsWidget, SliderValueWidget);
	}

	void FVertexAttributePaintToolDetailCustomization::MakeSelectionAddOperationRow(IDetailCategoryBuilder& EditValuesCategory) const
	{
		TSharedRef<SSpinBox<float>> AddValueWidget =
			SNew(SSpinBox<float>)
			.MinValue(-1)
			.MaxValue(1)
			.Value_Lambda([ToolProperties = ToolProperties]()
				{
					return ToolProperties.IsValid() ? ToolProperties->SelectionAddStrength : 1.f;
				})
			.OnValueChanged_Lambda([ToolProperties = ToolProperties](float NewValue)
				{
					if (ToolProperties.IsValid())
					{
						ToolProperties->SelectionAddStrength = NewValue;
					}
				})
			.OnValueCommitted_Lambda([Tool = Tool, ToolProperties = ToolProperties](float, ETextCommit::Type)
				{
					if (ToolProperties.IsValid()) { ToolProperties->SaveConfig(); }
					if (Tool.IsValid()) { Tool->SetFocusInViewport(); }
				})
			.ToolTipText(LOCTEXT("AddValueSliderToolTip", "Adjust the value to Add to the selected vertices."));

		TSharedRef<SWidget> AddButtonWidget =
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("AddValueButtonLabel", "Add"))
			.ToolTipText(LOCTEXT("AddValueButtonTooltip",
				"Add: applies the current value plus the flood value to the new weight.\n"
				"This operation applies to the currently selected vertices."))
			.OnClicked_Lambda([Tool = Tool, ToolProperties = ToolProperties]()
				{
					if (Tool.IsValid() && ToolProperties.IsValid())
					{
						Tool->ApplyValueToSelection(EMeshVertexAttributePaintToolEditOperation::Add, ToolProperties->SelectionAddStrength);
					}
					return FReply::Handled();
				});

		MakeSelectionOperationRow(EditValuesCategory, LOCTEXT("SelectionAddOperationRow", "Selection Add Operation"), AddButtonWidget, AddValueWidget);
	}

	void FVertexAttributePaintToolDetailCustomization::MakeSelectionReplaceOperationRow(IDetailCategoryBuilder& EditValuesCategory) const
	{
		TSharedRef<SSpinBox<float>> ReplaceValueWidget =
			SNew(SSpinBox<float>)
			.MinValue(0)
			.MaxValue(1)
			.Value_Lambda([ToolProperties = ToolProperties]()
				{
					return ToolProperties.IsValid() ? ToolProperties->SelectionReplaceValue : 1.f;
				})
			.OnValueChanged_Lambda([ToolProperties = ToolProperties](float NewValue)
				{
					if (ToolProperties.IsValid())
					{
						ToolProperties->SelectionReplaceValue = NewValue;
					}
				})
			.OnValueCommitted_Lambda([Tool = Tool, ToolProperties = ToolProperties](float, ETextCommit::Type)
				{
					if (ToolProperties.IsValid()) { ToolProperties->SaveConfig(); }
					if (Tool.IsValid()) { Tool->SetFocusInViewport(); }
				})
			.ToolTipText(LOCTEXT("ReplaceValueSliderToolTip", "Adjust the value to be replace on the selected vertices."));

		TSharedRef<SWidget> ReplaceButtonWidget =
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("ReplaceValueButtonLabel", "Replace"))
			.ToolTipText(LOCTEXT("ReplaceValueButtonTooltip",
				"Replace: the value of selected vertices is replaced by the specified value.\n"
				"This operation applies to the currently selected vertices."))
			.OnClicked_Lambda([Tool = Tool, ToolProperties = ToolProperties]()
				{
					if (Tool.IsValid() && ToolProperties.IsValid())
					{
						Tool->ApplyValueToSelection(EMeshVertexAttributePaintToolEditOperation::Replace, ToolProperties->SelectionReplaceValue);
					}
					return FReply::Handled();
				});

		MakeSelectionOperationRow(EditValuesCategory, LOCTEXT("SelectionReplaceOperationRow", "Selection Replace Operation"), ReplaceButtonWidget, ReplaceValueWidget);
	}

	void FVertexAttributePaintToolDetailCustomization::MakeSelectionInvertOperationRow(IDetailCategoryBuilder& EditValuesCategory) const
	{
		TSharedRef<SWidget> NoValueValueWidget = SNullWidget::NullWidget;

		TSharedRef<SWidget> ReplaceButtonWidget =
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("InvertValueButtonLabel", "Invert"))
			.ToolTipText(LOCTEXT("InvertValueButtonTooltip",
				"Invert: the value of selected vertices is replaced by opne minus the specified value.\n"
				"This operation applies to the currently selected vertices."))
			.OnClicked_Lambda([this]()
				{
					if (Tool.IsValid())
					{
						Tool->ApplyValueToSelection(EMeshVertexAttributePaintToolEditOperation::Invert, 0.0f);
					}
					return FReply::Handled();
				});

		MakeSelectionOperationRow(EditValuesCategory, LOCTEXT("SelectionInvertOperationRow", "Selection Invert Operation"), ReplaceButtonWidget, NoValueValueWidget);
	}

	void FVertexAttributePaintToolDetailCustomization::MakeSelectionRelaxOperationRow(IDetailCategoryBuilder& EditValuesCategory) const
	{
		TSharedRef<SSpinBox<float>> RelaxValueWidget =
			SNew(SSpinBox<float>)
			.MinValue(0)
			.MaxValue(1)
			.Value_Lambda([ToolProperties = ToolProperties]()
				{
					return ToolProperties.IsValid() ? ToolProperties->SelectionRelaxStrength : 0.5f;
				})
			.OnValueChanged_Lambda([ToolProperties = ToolProperties](float NewValue)
				{
					if (ToolProperties.IsValid())
					{
						ToolProperties->SelectionRelaxStrength = NewValue;
					}
				})
			.OnValueCommitted_Lambda([Tool = Tool, ToolProperties = ToolProperties](float, ETextCommit::Type)
				{
					if (ToolProperties.IsValid()) { ToolProperties->SaveConfig(); }
					if (Tool.IsValid()) { Tool->SetFocusInViewport(); }
				})
			.ToolTipText(LOCTEXT("RelaxValueSliderToolTip", "Amount to blend when relaxing  the values"));

		TSharedRef<SWidget> RelaxButtonWidget =
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("RelaxValueButtonLabel", "Relax"))
			.ToolTipText(LOCTEXT("RelaxValueButtonTooltip",
				"Relax: the value of each selected vertex is replaced by the average of it's neighbors. \n"
				"This smooths values across the mesh."))
			.OnClicked_Lambda([Tool = Tool, ToolProperties = ToolProperties]()
				{
					if (Tool.IsValid() && ToolProperties.IsValid())
					{
						Tool->ApplyValueToSelection(EMeshVertexAttributePaintToolEditOperation::Relax, ToolProperties->SelectionRelaxStrength);
					}
					return FReply::Handled();
				});

		MakeSelectionOperationRow(EditValuesCategory, LOCTEXT("SelectionRelaxOperationRow", "Selection Relax Operation"), RelaxButtonWidget, RelaxValueWidget);
	}

	void FVertexAttributePaintToolDetailCustomization::MakeSelectionPruneOperationRow(IDetailCategoryBuilder& EditValuesCategory) const
	{
		TSharedRef<SSpinBox<float>> PruneValueWidget =
			SNew(SSpinBox<float>)
			.MinValue(0.f)
			.MaxValue(1.f)
			.Value_Lambda([ToolProperties = ToolProperties]()
				{
					return ToolProperties.IsValid() ? ToolProperties->SelectionPruneThreshold : 0.01f;
				})
			.OnValueChanged_Lambda([ToolProperties = ToolProperties](float NewValue)
				{
					if (ToolProperties.IsValid())
					{
						ToolProperties->SelectionPruneThreshold = NewValue;
					}
				})
			.OnValueCommitted_Lambda([Tool = Tool, ToolProperties = ToolProperties](float, ETextCommit::Type)
				{
					if (ToolProperties.IsValid()) { ToolProperties->SaveConfig(); }
					if (Tool.IsValid()) { Tool->SetFocusInViewport(); }
				})
			.ToolTipText(LOCTEXT("PruneValueSliderToolTip", "Prune Threshold - Values below this threshold will be set to zero"));

		TSharedRef<SWidget> PruneButtonWidget =
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("PruneValuesButtonLabel", "Prune"))
			.ToolTipText(LOCTEXT("PruneButtonTooltip",
				"Zero values below the given threshold value.\n"
				"This command operates on the selected vertices."))
			.OnClicked_Lambda([Tool = Tool, ToolProperties = ToolProperties]()
				{
					if (Tool.IsValid() && ToolProperties.IsValid())
					{
						Tool->PruneSelection(ToolProperties->SelectionPruneThreshold);
					}
					return FReply::Handled();
				});

		MakeSelectionOperationRow(EditValuesCategory, LOCTEXT("SelectionPruneOperationRow", "Selection Prune Operation"), PruneButtonWidget, PruneValueWidget);
	}

	void FVertexAttributePaintToolDetailCustomization::MakeSelectionCopyAndPasteRow(IDetailCategoryBuilder& EditValuesCategory) const
	{
		// COPY/PASTE WEIGHTS category
		EditValuesCategory.AddCustomRow(LOCTEXT("CopyPasteValuesRow", "Copy Paste"), false)
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			.IsEnabled_Lambda([this]
				{
					return Tool.IsValid() ? Tool->HasSelection() : false;
				})

			+ SHorizontalBox::Slot()
			.Padding(2.f, WeightEditVerticalPadding)
			[
				SNew(SBox)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("CopyValuesButtonLabel", "Copy"))
					.ToolTip(MakeTooltipWithShortcut(LOCTEXT("CopyButtonTooltip", "Copy the average of the selected values to the clipboard.\nThis is designed to work with the Paste command."), UMeshVertexAttributePaintToolBase::CopyAverageValueInputChord))
					.OnClicked_Lambda([this]()
						{
							if (Tool.IsValid())
							{
								Tool->CopyAverageFromSelectionToClipboard();
							}
							return FReply::Handled();
						})
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(2.f, WeightEditVerticalPadding)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("PasteValuesButtonLabel", "Paste"))
				.ToolTip(MakeTooltipWithShortcut(LOCTEXT("PasteButtonTooltip", "Paste the value on the selected vertices.\nThis command requires the clipboard contain the selection average value from the Copy command."), UMeshVertexAttributePaintToolBase::PasteAverageValueInputChord))
				.OnClicked_Lambda([this]()
					{
						if (ToolProperties.IsValid())
						{
							Tool->PasteValueToSelectionFromClipboard();
						}
						return FReply::Handled();
					})
			]
		];
	}

	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::MakeMirrorAxisWidget() const
	{
		return SNew(SSegmentedControl<EAxis::Type>)
			.Value_Lambda([this]()
				{
					return Tool.IsValid() ? Tool->GetMirrorAxis() : EAxis::X;
				})
			.OnValueChanged_Lambda([this](EAxis::Type Mode)
				{
					if (Tool.IsValid())
					{
						Tool->SetMirrorAxis(Mode);
					}
				})
			+ SSegmentedControl<EAxis::Type>::Slot(EAxis::X)
			.Text(LOCTEXT("MirrorXLabel", "X"))
			.ToolTip(LOCTEXT("MirrorXLabelTooltip", "X: copies weights across the YZ plane."))
			+ SSegmentedControl<EAxis::Type>::Slot(EAxis::Y)
			.Text(LOCTEXT("MirrorYLabel", "Y"))
			.ToolTip(LOCTEXT("MirrorYLabelTooltip", "Y: copies weights across the XZ plane."))
			+ SSegmentedControl<EAxis::Type>::Slot(EAxis::Z)
			.Text(LOCTEXT("MirrorZLabel", "Z"))
			.ToolTip(LOCTEXT("MirrorZLabelTooltip", "Z: copies weights across the XY plane."))
			;
	}

	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::MakeMirrorShowAndAxisWidget() const
	{
		TSharedRef<SWidget> ShowButton = SNew(SCheckBox)
			.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
			.ToolTipText(LOCTEXT("MirrorPlaneShowHide", "Toggle the mirror plane visibility in the viewport."))
			.HAlign(HAlign_Center)
			.Padding(2)
			.IsChecked_Lambda([this]()-> ECheckBoxState
			{
				return (Tool.IsValid() && Tool->IsMirrorPlaneWidgetVisible())? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([this](const ECheckBoxState NewState)
			{
				if (Tool.IsValid())
				{
					Tool->SetMirrorPlaneWidgetVisible(NewState == ECheckBoxState::Checked);
				}
			})
			[
				SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush("Level.VisibleIcon16x"))
			];

		auto AxisWidgetIsEnabled = [this]()
			{
				return (Tool.IsValid() && ((Tool->IsInBrushMode() && Tool->IsBrushMirroringEnabled()) || !Tool->IsInBrushMode()));
			};

		return SNew(SHorizontalBox)
			.IsEnabled_Lambda(AxisWidgetIsEnabled)
			+ SHorizontalBox::Slot()
			.FillWidth(.25f)
			[
				ShowButton
			]
			+ SHorizontalBox::Slot()
			.FillWidth(.75f)
			[
				MakeMirrorAxisWidget()
			];
	}

	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::MakeMirrorDirectionButtonsWidget() const
	{
		return SNew(SSegmentedControl<EMeshVertexAttributePaintToolMirrorDirection>)
			.ToolTipText(LOCTEXT("MirrorDirectionTooltip", "The direction that determines what side of the plane to copy weights from."))
			.Value_Lambda([this]()
				{
					return ToolProperties.IsValid() ? ToolProperties->MirrorProperties.MirrorDirection : EMeshVertexAttributePaintToolMirrorDirection::PositiveToNegative;
				})
			.OnValueChanged_Lambda([this](EMeshVertexAttributePaintToolMirrorDirection Mode)
				{
					if (ToolProperties.IsValid())
					{
						ToolProperties->MirrorProperties.MirrorDirection = Mode;
						if (Tool.IsValid())
						{
							Tool->SetFocusInViewport();
						}
					}
				})
			+ SSegmentedControl<EMeshVertexAttributePaintToolMirrorDirection>::Slot(EMeshVertexAttributePaintToolMirrorDirection::PositiveToNegative)
			.Text(LOCTEXT("MirrorPosToNegLabel", "+ to -"))
			.ToolTipWidget(MakeTooltipWithShortcut(LOCTEXT("MirrorDirectionPosToNegTooltip", "Mirror the values from the positive side of the axis to the negative side."), UMeshVertexAttributePaintToolBase::InvertMirrorDirectionInputChord))
			+ SSegmentedControl<EMeshVertexAttributePaintToolMirrorDirection>::Slot(EMeshVertexAttributePaintToolMirrorDirection::NegativeToPositive)
			.Text(LOCTEXT("MirrorNegToPosLabel", "- to +"))
			.ToolTipWidget(MakeTooltipWithShortcut(LOCTEXT("MirrorDirectionNegToPosTooltip", "Mirror the values from the negative side of the axis to the positive side."), UMeshVertexAttributePaintToolBase::InvertMirrorDirectionInputChord))
			;
	}

	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::MakeMirrorPlaneWidget(bool bShowDirection) const
	{
		TSharedRef<SWidget> XYZButtonsWidget = MakeMirrorShowAndAxisWidget();
		TSharedRef<SWidget> MirrorDirectionButtonsWidget =
			bShowDirection
			? MakeMirrorDirectionButtonsWidget()
			: SNullWidget::NullWidget;

		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(WeightEditingLabelsPercent)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MirrorPlaneLabel", "Mirror Plane"))
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.ToolTipText(LOCTEXT("MirrorPlaneTooltip", "The plane to copy weights across."))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(0.5f)
				[
					XYZButtonsWidget
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.5f)
				[
					MirrorDirectionButtonsWidget
				]
			];
	}

	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::MakeMirrorObjectSpaceCheckboxWidget() const
	{
		return SNew(SCheckBox)
			.IsChecked_Lambda([ToolProperties = ToolProperties]()
				{
					const bool bIsChecked = (ToolProperties.IsValid() && ToolProperties->MirrorProperties.bObjectSpace);
					return bIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			.OnCheckStateChanged_Lambda([Tool = Tool, ToolProperties = ToolProperties](ECheckBoxState InCheckBoxState)
				{
					if (ToolProperties.IsValid())
					{
						ToolProperties->MirrorProperties.bObjectSpace = (InCheckBoxState == ECheckBoxState::Checked);
						if (Tool.IsValid())
						{
							Tool->SetFocusInViewport();
						}
					}
				});
	}

	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::MakeMirrorObjectSpaceWidget() const
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(WeightEditingLabelsPercent)
			[
				SNew(STextBlock)
					.Text(LOCTEXT("MirrorObjectSpaceLabel", "Object Space"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("MirrorObjectSpaceTooltip", "When checked, the planes will be centered on the object bounds."))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				MakeMirrorObjectSpaceCheckboxWidget()
			];
	}

	TSharedRef<SWidget> FVertexAttributePaintToolDetailCustomization::MakeMirrorButton() const
	{
		return SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("MirrorWeightsButtonLabel", "Mirror"))
			.ToolTip(MakeTooltipWithShortcut(LOCTEXT("MirrorButtonTooltip", "Values are copied across the given plane in the given direction.\nThis command operates on the selected vertices."), UMeshVertexAttributePaintToolBase::MirrorValuesInputChord))
			.OnClicked_Lambda([this]()
				{
					if (Tool.IsValid())
					{
						Tool->MirrorValues();
					}
					return FReply::Handled();
				});
	}

	void FVertexAttributePaintToolDetailCustomization::MakeSelectionMirrorOperationRow(IDetailCategoryBuilder& EditValuesCategory) const
	{
		TSharedRef<SWidget> MirrorPlaneWidget = MakeMirrorPlaneWidget(/*bShowDirection*/ true);
		TSharedRef<SWidget> ObjectSpaceWidget = MakeMirrorObjectSpaceWidget();
		TSharedRef<SWidget> MirrorButtonWidget = MakeMirrorButton();

		constexpr float HorizontalPadding = 2.0f;
		constexpr float VerticalPadding = 4.0f;

		// MIRROR WEIGHTS category
		EditValuesCategory.AddCustomRow(LOCTEXT("MirrorWeightsRow", "Mirror"), false)
			.WholeRowContent()
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(HorizontalPadding, VerticalPadding)
					[
						MirrorPlaneWidget
					]
					+ SVerticalBox::Slot()
					.Padding(HorizontalPadding, VerticalPadding)
					[
						ObjectSpaceWidget
					]
					+ SVerticalBox::Slot()
					.Padding(0.f, WeightEditVerticalPadding)
					[
						SNew(SBox)
						.IsEnabled_Lambda([this]
							{
								return Tool.IsValid() ? Tool->HasSelection() : false;
							})
						[
							MirrorButtonWidget
						]
					]
			];
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FVertexAttributePaintToolDetailCustomization::AddSelectionUI(IDetailLayoutBuilder& DetailBuilder) const
	{
		// custom display of selection editing tools
		IDetailCategoryBuilder& EditSelectionCategory = DetailBuilder.EditCategory("Edit Selection", FText::GetEmpty(), ECategoryPriority::Important);
		EditSelectionCategory.InitiallyCollapsed(false);
		{
			// create a toolbar for the selection filter [ Vtx |  Edges | Faces ] (Isolate)
			AddSelectionElementsRow(EditSelectionCategory);
		}
	
		IDetailCategoryBuilder& EditValuesCategory = DetailBuilder.EditCategory("Edit Values", FText::GetEmpty(), ECategoryPriority::Important);
		EditValuesCategory.InitiallyCollapsed(false);
		{
			AddEmptySelectionWarningRow(EditValuesCategory);

			MakeSelectionAddMultiplySliderOperationRow(EditValuesCategory);

			MakeSelectionAddOperationRow(EditValuesCategory);

			MakeSelectionReplaceOperationRow(EditValuesCategory);

			MakeSelectionInvertOperationRow(EditValuesCategory);

			MakeSelectionRelaxOperationRow(EditValuesCategory);

			MakeSelectionMirrorOperationRow(EditValuesCategory);

			MakeSelectionPruneOperationRow(EditValuesCategory);

			MakeSelectionCopyAndPasteRow(EditValuesCategory);
		}
	}

	void FVertexAttributePaintToolDetailCustomization::HideProperty(IDetailLayoutBuilder& DetailBuilder, FName PropertyName) const
	{
		TSharedRef<IPropertyHandle> Property = DetailBuilder.GetProperty(PropertyName);
		DetailBuilder.HideProperty(Property);
	}
}

#undef LOCTEXT_NAMESPACE
