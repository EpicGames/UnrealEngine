// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/MeshVertexPaintToolCustomizations.h"

#include "MeshVertexPaintTool.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ModelingToolsEditorModeStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"

TSharedRef<IDetailCustomization> FVertexPaintBasicPropertiesDetails::MakeInstance()
{
	return MakeShareable(new FVertexPaintBasicPropertiesDetails);
}

void FVertexPaintBasicPropertiesDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle> PaintHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexPaintBasicProperties, PaintColor), UVertexPaintBasicProperties::StaticClass());
	ensure(PaintHandle->IsValidHandle());
	
	TSharedPtr<IPropertyHandle> PaintPressureHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexPaintBasicProperties, bIsPaintPressureEnabled), UVertexPaintBasicProperties::StaticClass());
	ensure(PaintPressureHandle->IsValidHandle());
	PaintPressureHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> EraseHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexPaintBasicProperties, EraseColor), UVertexPaintBasicProperties::StaticClass());
	ensure(EraseHandle->IsValidHandle());

	TSharedPtr<IPropertyHandle> ErasePressureHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexPaintBasicProperties, bIsErasePressureEnabled), UVertexPaintBasicProperties::StaticClass());
	ensure(ErasePressureHandle->IsValidHandle());
	ErasePressureHandle->MarkHiddenByCustomization();

	BuildPaintPressureWidget(DetailBuilder, PaintHandle, PaintPressureHandle);
	BuildPaintPressureWidget(DetailBuilder, EraseHandle, ErasePressureHandle);
}

void FVertexPaintBasicPropertiesDetails::BuildPaintPressureWidget(IDetailLayoutBuilder& DetailBuilder,
	const TSharedPtr<IPropertyHandle>& PropHandle, TSharedPtr<IPropertyHandle> EnablePressureSensitivityHandle)
{
	IDetailPropertyRow* DetailRow = DetailBuilder.EditDefaultProperty(PropHandle);
	TSharedPtr<SWidget> NameWidget, ValueWidget;
	DetailRow->GetDefaultWidgets(NameWidget, ValueWidget);

	TSharedPtr<SHorizontalBox> ValueContent;

	DetailRow->CustomWidget()
	.NameContent() // property text
	[
		NameWidget->AsShared()
	]
	.ValueContent() // normal property widget and pressure sensitivity toggle
	[
		SAssignNew(ValueContent, SHorizontalBox)
	];

	// always add usual widget
	ValueContent->AddSlot()
	[
		ValueWidget->AsShared()
	];

	// add pressure sensitivity toggle
	ValueContent->AddSlot()
	.AutoWidth()
	[
		SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.Padding(FMargin(4, 2))
		.ToolTipText(EnablePressureSensitivityHandle->GetToolTipText())
		.HAlign(HAlign_Center)
		.OnCheckStateChanged_Lambda([EnablePressureSensitivityHandle](const ECheckBoxState NewState)
		{
			EnablePressureSensitivityHandle->SetValue(NewState == ECheckBoxState::Checked);
		})
		.IsChecked_Lambda([EnablePressureSensitivityHandle]() -> ECheckBoxState
		{
			bool bSet;
			EnablePressureSensitivityHandle->GetValue(bSet);
			return bSet ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0))
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FModelingToolsEditorModeStyle::Get()->GetBrush("BrushIcons.PressureSensitivity.Small"))
			]
		]
	];
}