// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/MeshVertexSculptToolCustomizations.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Texture2D.h"
#include "Widgets/Input/SCheckBox.h"

#include "CollectionManagerModule.h"
#include "ICollectionManager.h"

#include "ModelingWidgets/SComboPanel.h"
#include "ModelingWidgets/SToolInputAssetComboPanel.h"
#include "ModelingWidgets/ModelingCustomizationUtil.h"
#include "DetailsCustomizations/ModelingToolsBrushSizeCustomization.h"

#include "ModelingToolsEditorModeStyle.h"
#include "ModelingToolsEditorModeSettings.h"

#include "MeshVertexSculptTool.h"
#include "ModelingWidgets/SculptLayersController.h"
#include "ModelingWidgets/SMeshLayersStack.h"
#include "Properties/MeshSculptLayerProperties.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Sculpting/KelvinletBrushOp.h"
#include "Sculpting/MeshInflateBrushOps.h"
#include "Sculpting/MeshMoveBrushOps.h"
#include "Sculpting/MeshPinchBrushOps.h"
#include "Sculpting/MeshPlaneBrushOps.h"
#include "Sculpting/MeshSculptBrushOps.h"
#include "Sculpting/MeshSmoothingBrushOps.h"
#include "Sculpting/MeshEraseSculptLayerBrushOps.h"

using namespace UE::ModelingUI;

#define LOCTEXT_NAMESPACE "MeshVertexSculptToolCustomizations"

TSharedRef<IDetailCustomization> FSculptBrushPropertiesDetails::MakeInstance()
{
	return MakeShareable(new FSculptBrushPropertiesDetails);
}


void FSculptBrushPropertiesDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);
	USculptBrushProperties* BrushProperties = Cast<USculptBrushProperties>(ObjectsBeingCustomized[0]);

	// If we show falloff, it gets placed on the same line as flow/spacing. Otherwise, flow/spacing will get their own, uncustomized, lines
	if (BrushProperties->bShowFalloff && (BrushProperties->bShowSpacing || BrushProperties->bShowFlowRate))
	{
		TSharedPtr<IPropertyHandle> FalloffHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USculptBrushProperties, BrushFalloffAmount), USculptBrushProperties::StaticClass());
		ensure(FalloffHandle->IsValidHandle());
		FalloffHandle->MarkHiddenByCustomization();
		TSharedPtr<SDynamicNumericEntry::FDataSource> FalloffSource = SDynamicNumericEntry::MakeSimpleDataSource(
			FalloffHandle, TInterval<float>(0.0f, 1.0f), TInterval<float>(0.0f, 1.0f));

		// Note: the below assumes that edit conditions are set up in such a way that spacing and flow rate are mutually exclusive,
		//  otherwise falloff (if shown) would show up twice. However, note that bShowSpacing and bShowFlowRate are NOT mutually
		//  exclusive. We customize each one, and then their individual visibility controls the visibility of our customized row. 
		if (BrushProperties->bShowSpacing)
		{
			TSharedPtr<IPropertyHandle> SpacingHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USculptBrushProperties, Spacing), USculptBrushProperties::StaticClass());
			ensure(SpacingHandle->IsValidHandle());
			
			TSharedPtr<SDynamicNumericEntry::FDataSource> SpacingSource = SDynamicNumericEntry::MakeSimpleDataSource(
			SpacingHandle, TInterval<float>(0.0f, 1000.0f), TInterval<float>(0.0f, 4.0f));

			DetailBuilder.EditDefaultProperty(SpacingHandle)->CustomWidget()
				.OverrideResetToDefault(FResetToDefaultOverride::Hide())
				.WholeRowContent()
			[
				MakeTwoWidgetDetailRowHBox(
					MakeFixedWidthLabelSliderHBox(FalloffHandle, FalloffSource, FSculptToolsUIConstants::SculptShortLabelWidth),
					MakeFixedWidthLabelSliderHBox(SpacingHandle, SpacingSource, FSculptToolsUIConstants::SculptShortLabelWidth)
				)
			];
		}
		if (BrushProperties->bShowFlowRate)
		{
			TSharedPtr<IPropertyHandle> FlowRateHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USculptBrushProperties, FlowRate), USculptBrushProperties::StaticClass());
			ensure(FlowRateHandle->IsValidHandle());
		
			TSharedPtr<SDynamicNumericEntry::FDataSource> FlowRateSource = SDynamicNumericEntry::MakeSimpleDataSource(
				FlowRateHandle, TInterval<float>(0.0f, 1.0f), TInterval<float>(0.0f, 1.0f));

			DetailBuilder.EditDefaultProperty(FlowRateHandle)->CustomWidget()
				.OverrideResetToDefault(FResetToDefaultOverride::Hide())
				.WholeRowContent()
			[
				MakeTwoWidgetDetailRowHBox(
					MakeFixedWidthLabelSliderHBox(FalloffHandle, FalloffSource, FSculptToolsUIConstants::SculptShortLabelWidth),
					MakeFixedWidthLabelSliderHBox(FlowRateHandle, FlowRateSource, FSculptToolsUIConstants::SculptShortLabelWidth)
				)
			];
		}
	}

	TSharedPtr<IPropertyHandle> LazynessHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USculptBrushProperties, Lazyness), USculptBrushProperties::StaticClass());
	ensure(LazynessHandle->IsValidHandle());

	TSharedPtr<IPropertyHandle> HitBackFacesHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USculptBrushProperties, bHitBackFaces), USculptBrushProperties::StaticClass());
	ensure(HitBackFacesHandle->IsValidHandle());
	HitBackFacesHandle->MarkHiddenByCustomization();

	// todo: 0-100 mapping
	TSharedPtr<SDynamicNumericEntry::FDataSource> LazynessSource = SDynamicNumericEntry::MakeSimpleDataSource(
		LazynessHandle, TInterval<float>(0.0f, 1.0f), TInterval<float>(0.0f, 1.0f));

	DetailBuilder.EditDefaultProperty(LazynessHandle)->CustomWidget()
		.OverrideResetToDefault(FResetToDefaultOverride::Hide())
		.WholeRowContent()
	[
		MakeTwoWidgetDetailRowHBox(
			MakeFixedWidthLabelSliderHBox(LazynessHandle, LazynessSource, FSculptToolsUIConstants::SculptShortLabelWidth),
			MakeBoolToggleButton(HitBackFacesHandle, LOCTEXT("HitBackFacesText", "Hit Back Faces") )
		)
	];

}

FMeshSculptLayerPropertiesDetails::FMeshSculptLayerPropertiesDetails()
{
	Controller = MakeShared<FSculptLayersController>();
}

FMeshSculptLayerPropertiesDetails::~FMeshSculptLayerPropertiesDetails()
{
	if (SculptProperties.IsValid())
	{
		SculptProperties->OnMeshLayersPanelRequestRebuild.Remove(SculptUpdatesHandle);
	}
}

TSharedRef<IDetailCustomization> FMeshSculptLayerPropertiesDetails::MakeInstance()
{
	return MakeShareable(new FMeshSculptLayerPropertiesDetails);
}

void FMeshSculptLayerPropertiesDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);
	UMeshSculptLayerProperties* SculptLayersProperties = Cast<UMeshSculptLayerProperties>(ObjectsBeingCustomized[0]);
	SculptProperties = SculptLayersProperties;
	Controller->SetProperties(SculptLayersProperties);

	TSharedPtr<IPropertyHandle> ActiveLayerHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshSculptLayerProperties, ActiveLayer), UMeshSculptLayerProperties::StaticClass());
	ensure(ActiveLayerHandle->IsValidHandle());
	
	TSharedPtr<IPropertyHandle> LayerWeightsHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshSculptLayerProperties, LayerWeights), UMeshSculptLayerProperties::StaticClass());
	ensure(LayerWeightsHandle->IsValidHandle());
	LayerWeightsHandle->MarkHiddenByCustomization();
	
	DetailBuilder
	.EditDefaultProperty(ActiveLayerHandle)->CustomWidget()
	.OverrideResetToDefault(FResetToDefaultOverride::Hide())
	.WholeRowContent()
	[
		SNew(SMeshLayersStack, Controller)
	];

	// verify invariant that this is always a fresh details object (if this were not the case, would need to unbind delegate handle above.)
	ensureMsgf(!SculptUpdatesHandle.IsValid(), TEXT("Unexpected re-use of detail customization object"));
	// ensure this stack is rebuilt when Mesh Layers are updated elsewhere in the editor
	TWeakPtr<IMeshLayersController> WeakController = Controller;
	SculptUpdatesHandle = SculptLayersProperties->OnMeshLayersPanelRequestRebuild.AddLambda([WeakController]() {
		if (TSharedPtr<IMeshLayersController> PinnedController = WeakController.Pin())
		{
			PinnedController->RefreshLayersStackView();
		}
	});
}

template<typename BrushClass>
TSharedRef<IDetailCustomization> FMeshSculptBrushOpPropertiesDetails<BrushClass>::MakeInstance()
{
	return MakeShareable(new FMeshSculptBrushOpPropertiesDetails<BrushClass>);
}

template<typename BrushClass>
void FMeshSculptBrushOpPropertiesDetails<BrushClass>::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);

	// brush op properties
	BrushClass* BrushOpProperties = Cast<BrushClass>(ObjectsBeingCustomized[0]);

	// toggles pressure sensitivity for brush strength
	const TSharedPtr<IPropertyHandle> EnablePressureSensitivity = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshSculptBrushOpProps, bIsStrengthPressureEnabled), UMeshSculptBrushOpProps::StaticClass());
	ensure(EnablePressureSensitivity->IsValidHandle());
	EnablePressureSensitivity->MarkHiddenByCustomization(); // always hidden

	// do the tool & brush support pressure sensitivity for brush strength?
	const bool bSupportsPressureSensitivity = BrushOpProperties->SupportsStrengthPressure();

	// controls brush strength
	const TSharedPtr<IPropertyHandle> StrengthHandle = DetailBuilder.GetProperty("Strength", BrushClass::StaticClass());

	if (bSupportsPressureSensitivity && ensure(StrengthHandle->IsValidHandle()))
	{
		// build customized widget representing brush strength & pressure sensitivity toggle (when applicable)
		BuildStrengthPressureWidget(DetailBuilder, StrengthHandle, EnablePressureSensitivity);
	}
}

template<typename BrushClass>
void FMeshSculptBrushOpPropertiesDetails<BrushClass>::BuildStrengthPressureWidget(IDetailLayoutBuilder& DetailBuilder, const TSharedPtr<IPropertyHandle>& StrengthHandle, TSharedPtr<IPropertyHandle> EnablePressureSensitivityHandle)
{
	IDetailPropertyRow* DetailRow = DetailBuilder.EditDefaultProperty(StrengthHandle);
	TSharedPtr<SWidget> NameWidget, ValueWidget;
	DetailRow->GetDefaultWidgets(NameWidget, ValueWidget);

	TSharedPtr<SHorizontalBox> ValueContent;

	DetailRow->CustomWidget()
	.NameContent() // strength text
	[
		NameWidget->AsShared()
	]
	.ValueContent() // strength sspinbox and pressure sensitivity toggle, when applicable
	[
		SAssignNew(ValueContent, SHorizontalBox)
	];

	// always add slider/SSpinbox to control strength, as usual
	ValueContent->AddSlot()
	[
		ValueWidget->AsShared()
	];

	// add pressure sensitivity toggle
	ValueContent->AddSlot()
	.AutoWidth()
	[
		SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.Padding(FMargin(4, 2))
		.ToolTipText(EnablePressureSensitivityHandle->GetToolTipText())
		.HAlign(HAlign_Center)
		.OnCheckStateChanged_Lambda([EnablePressureSensitivityHandle](const ECheckBoxState NewState)
		{
			EnablePressureSensitivityHandle->SetValue(NewState == ECheckBoxState::Checked);
		})
		.IsChecked_Lambda([EnablePressureSensitivityHandle]() -> ECheckBoxState
		{
			bool bSet;
			EnablePressureSensitivityHandle->GetValue(bSet);
			return bSet ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0))
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FModelingToolsEditorModeStyle::Get()->GetBrush("BrushIcons.PressureSensitivity.Small"))
			]
		]
	];
}

/* Explicitly instantiate the types which are supported */

template class FMeshSculptBrushOpPropertiesDetails<UPinchBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UInflateBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<USmoothBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<USmoothFillBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UFlattenBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UEraseBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UStandardSculptBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UViewAlignedSculptBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<USculptMaxBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UFixedPlaneBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UViewAlignedPlaneBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UPlaneBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UMoveBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<USecondarySmoothBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UScaleKelvinletBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UPullKelvinletBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<USharpPullKelvinletBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UTwistKelvinletBrushOpProps>;
template class FMeshSculptBrushOpPropertiesDetails<UEraseSculptLayerBrushOpProps>;

/***********************************************************/

TSharedRef<IDetailCustomization> FVertexBrushSculptPropertiesDetails::MakeInstance()
{
	return MakeShareable(new FVertexBrushSculptPropertiesDetails);
}

FVertexBrushSculptPropertiesDetails::~FVertexBrushSculptPropertiesDetails()
{
	if (TargetTool.IsValid())
	{
		TargetTool.Get()->OnDetailsPanelRequestRebuild.Remove(FalloffTypeUpdateHandle);
	}
}

void FVertexBrushSculptPropertiesDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle> BrushTypeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushSculptProperties, PrimaryBrushID), UVertexBrushSculptProperties::StaticClass());
	ensure(BrushTypeHandle->IsValidHandle());

	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);
	UVertexBrushSculptProperties* BrushProperties = CastChecked<UVertexBrushSculptProperties>(ObjectsBeingCustomized[0]);
	UMeshVertexSculptTool* Tool = BrushProperties->Tool.Get();
	TargetTool = Tool;

	const TSet<UMeshSculptToolBase::FBrushTypeInfo> BrushTypeInfos = Tool->GetAvailableBrushTypes();
	int32 CurrentBrushType = (int32)BrushProperties->PrimaryBrushID;
	int32 CurrentBrushTypeIndex = 0;

	TArray<TSharedPtr<SComboPanel::FComboPanelItem>> BrushTypeItems;
	for (const UMeshSculptToolBase::FBrushTypeInfo& BrushTypeInfo : BrushTypeInfos)
	{
		TSharedPtr<SComboPanel::FComboPanelItem> NewBrushTypeItem = MakeShared<SComboPanel::FComboPanelItem>();
		NewBrushTypeItem->Name = BrushTypeInfo.Name;
		NewBrushTypeItem->Identifier = BrushTypeInfo.Identifier;
		const FString* SourceBrushName = FTextInspector::GetSourceString(BrushTypeInfo.Name);
		NewBrushTypeItem->Icon = FModelingToolsEditorModeStyle::Get()->GetBrush( FName(TEXT("BrushTypeIcons.") + (*SourceBrushName) ));
		if (NewBrushTypeItem->Identifier == CurrentBrushType)
		{
			CurrentBrushTypeIndex = BrushTypeItems.Num();
		}
		BrushTypeItems.Add(NewBrushTypeItem);
	}

	float ComboIconSize = 60;
	float FlyoutIconSize = 100;
	float FlyoutWidth = 840;

	TSharedPtr<SComboPanel> BrushTypeCombo = SNew(SComboPanel)
		.ToolTipText(BrushTypeHandle->GetToolTipText())
		.ComboButtonTileSize(FVector2D(ComboIconSize, ComboIconSize))
		.FlyoutTileSize(FVector2D(FlyoutIconSize, FlyoutIconSize))
		.FlyoutSize(FVector2D(FlyoutWidth, 1))
		.ListItems(BrushTypeItems)
		.IsEnabled_Lambda([this]()
		{
			if (const UMeshVertexSculptTool* Tool = TargetTool.Get())
			{
				return Tool->CanUpdateBrushType();
			}
			return false;
		})
		.OnSelectionChanged_Lambda([this](TSharedPtr<SComboPanel::FComboPanelItem> NewSelectedItem) {
			if ( UMeshVertexSculptTool* Tool = TargetTool.Get() )
			{
				if (Tool->CanUpdateBrushType())
				{
					Tool->SetActiveBrushType(NewSelectedItem->Identifier);
				}
			}
		})
		.FlyoutHeaderText(LOCTEXT("BrushesHeader", "Brush Types"))
		.InitialSelectionIndex(CurrentBrushTypeIndex);


	TSharedPtr<IPropertyHandle> FalloffTypeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushSculptProperties, PrimaryFalloffType), UVertexBrushSculptProperties::StaticClass());
	ensure(FalloffTypeHandle->IsValidHandle());

	const TArray<UMeshSculptToolBase::FFalloffTypeInfo>& FalloffTypeInfos = Tool->GetRegisteredPrimaryFalloffTypes();
	int32 CurrentFalloffType = (int32)BrushProperties->PrimaryFalloffType;
	int32 CurrentFalloffTypeIndex = 0;

	TArray<TSharedPtr<SComboPanel::FComboPanelItem>> FalloffTypeItems;
	for (const UMeshSculptToolBase::FFalloffTypeInfo& FalloffTypeInfo : FalloffTypeInfos)
	{
		TSharedPtr<SComboPanel::FComboPanelItem> NewFalloffTypeItem = MakeShared<SComboPanel::FComboPanelItem>();
		NewFalloffTypeItem->Name = FalloffTypeInfo.Name;
		NewFalloffTypeItem->Identifier = FalloffTypeInfo.Identifier;
		NewFalloffTypeItem->Icon = FModelingToolsEditorModeStyle::Get()->GetBrush( FName(TEXT("BrushFalloffIcons.") + FalloffTypeInfo.StringIdentifier) );
		if (NewFalloffTypeItem->Identifier == CurrentFalloffType)
		{
			CurrentFalloffTypeIndex = FalloffTypeItems.Num();
		}
		FalloffTypeItems.Add(NewFalloffTypeItem);
	}

	FalloffTypeCombo = SNew(SComboPanel)		
		.ToolTipText(FalloffTypeHandle->GetToolTipText())
		.ComboButtonTileSize(FVector2D(18, 18))
		.FlyoutTileSize(FVector2D(FlyoutIconSize, FlyoutIconSize))
		.FlyoutSize(FVector2D(FlyoutWidth, 1))
		.ListItems(FalloffTypeItems)
		.ComboDisplayType(SComboPanel::EComboDisplayType::IconAndLabel)
		.OnSelectionChanged_Lambda([this](TSharedPtr<SComboPanel::FComboPanelItem> NewSelectedFalloffTypeItem) {
			if ( TargetTool.IsValid() )
			{
				TargetTool.Get()->SetActiveFalloffType(NewSelectedFalloffTypeItem->Identifier);
			}
		})
		.FlyoutHeaderText(LOCTEXT("FalloffsHeader", "Falloff Types"))
		.InitialSelectionIndex(CurrentFalloffTypeIndex);

	FalloffTypeUpdateHandle = TargetTool.Get()->OnDetailsPanelRequestRebuild.AddLambda([this]() {
			UVertexBrushSculptProperties* BrushProperties = CastChecked<UVertexBrushSculptProperties>(ObjectsBeingCustomized[0]);
			const TArray<UMeshSculptToolBase::FFalloffTypeInfo>& FalloffTypeInfos = TargetTool->GetRegisteredPrimaryFalloffTypes();
			int32 CurrentFalloffType = (int32)BrushProperties->PrimaryFalloffType;
			FalloffTypeCombo->SetSelectionIndex(CurrentFalloffType);
	});

	//DetailBuilder.HideProperty(FalloffTypeHandle);
	FalloffTypeHandle->MarkHiddenByCustomization();


	TSharedPtr<IPropertyHandle> BrushFilterHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushSculptProperties, BrushFilter), UVertexBrushSculptProperties::StaticClass());
	BrushFilterHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> FreezeTargetHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushSculptProperties, bFreezeTarget), UVertexBrushSculptProperties::StaticClass());
	FreezeTargetHandle->MarkHiddenByCustomization();


	DetailBuilder.EditDefaultProperty(BrushTypeHandle)->CustomWidget()
		.OverrideResetToDefault(FResetToDefaultOverride::Hide())
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0, ModelingUIConstants::DetailRowVertPadding))
			.AutoWidth()
			[
				SNew(SBox)
				.HeightOverride(ComboIconSize+14)
				[
					BrushTypeCombo->AsShared()
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(ModelingUIConstants::MultiWidgetRowHorzPadding,ModelingUIConstants::DetailRowVertPadding,0,ModelingUIConstants::MultiWidgetRowHorzPadding))
			.FillWidth(1.0f)
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(FMargin(0))
					.AutoHeight()
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								WrapInFixedWidthBox(FalloffTypeHandle->CreatePropertyNameWidget(), FSculptToolsUIConstants::SculptShortLabelWidth)
							]
							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							[
								FalloffTypeCombo->AsShared()
							]
					]

					+ SVerticalBox::Slot()
					.Padding(FMargin(0, ModelingUIConstants::DetailRowVertPadding, 0, 0))
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							WrapInFixedWidthBox(BrushFilterHandle->CreatePropertyNameWidget(), FSculptToolsUIConstants::SculptShortLabelWidth)
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							MakeRegionFilterWidget()->AsShared()
						]
					]

					+ SVerticalBox::Slot()
					.Padding(FMargin(0, ModelingUIConstants::DetailRowVertPadding, 0, 0))
					.AutoHeight()
					[
						MakeFreezeTargetWidget()->AsShared()
					]
			]
		];

}


TSharedPtr<SWidget> FVertexBrushSculptPropertiesDetails::MakeRegionFilterWidget()
{
	static const FText RegionFilterLabels[] = {
		LOCTEXT("RegionFilterNone", "Vol"),  
		LOCTEXT("RegionFilterComponent", "Cmp"),  
		LOCTEXT("RegionFilterPolyGroup", "Grp")
	};
	static const FText RegionFilterTooltips[] = {
		LOCTEXT("RegionFilterNoneTooltip", "Do not filter brush area, include all triangles in brush sphere"),
		LOCTEXT("RegionFilterComponentTooltip", "Only apply brush to triangles in the same connected mesh component/island"),
		LOCTEXT("RegionFilterPolygroupTooltip", "Only apply brush to triangles with the same PolyGroup"),
	};

	auto MakeRegionFilterButton = [this](EMeshVertexSculptBrushFilterType FilterType)
	{
		return SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.Padding(FMargin(2, 2))
			.HAlign(HAlign_Center)
			.ToolTipText( RegionFilterTooltips[(int32)FilterType] )
			.OnCheckStateChanged_Lambda([this, FilterType](ECheckBoxState State) {
				if (TargetTool.IsValid() && State == ECheckBoxState::Checked)
				{
					TargetTool.Get()->SetRegionFilterType( static_cast<int32>(FilterType) );
				}
			})
			.IsChecked_Lambda([this, FilterType]() {
				return (TargetTool.IsValid() && TargetTool.Get()->SculptProperties->BrushFilter == FilterType) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0))
				.AutoWidth()
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					.Text( RegionFilterLabels[(int32)FilterType] )
				]
			];
	};


	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			MakeRegionFilterButton(EMeshVertexSculptBrushFilterType::None)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			MakeRegionFilterButton(EMeshVertexSculptBrushFilterType::Component)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			MakeRegionFilterButton(EMeshVertexSculptBrushFilterType::PolyGroup)
		];
}


TSharedPtr<SWidget> FVertexBrushSculptPropertiesDetails::MakeFreezeTargetWidget()
{
	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.Padding(FMargin(0, 4))
		.ToolTipText(LOCTEXT("FreezeTargetTooltip", "When Freeze Target is toggled on, the Brush Target Surface will be Frozen in its current state, until toggled off. Brush strokes will be applied relative to the Target Surface, for applicable Brushes"))
		.HAlign(HAlign_Center)
		.OnCheckStateChanged(this, &FVertexBrushSculptPropertiesDetails::OnSetFreezeTarget)
		.IsChecked_Lambda([this] { return IsFreezeTargetEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; } )
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0))
			.AutoWidth()
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				.Text_Lambda( [this] { return IsFreezeTargetEnabled() ? LOCTEXT("UnFreezeTarget", "UnFreeze Target") : LOCTEXT("FreezeTarget", "Freeze Target"); } )
			]
		];
}




FReply FVertexBrushSculptPropertiesDetails::OnToggledFreezeTarget()
{
	if ( TargetTool.IsValid() )
	{
		TargetTool.Get()->SculptProperties->bFreezeTarget = !TargetTool.Get()->SculptProperties->bFreezeTarget;
	}
	return FReply::Handled();
}

void FVertexBrushSculptPropertiesDetails::OnSetFreezeTarget(ECheckBoxState State)
{
	if ( TargetTool.IsValid() )
	{
		TargetTool.Get()->SculptProperties->bFreezeTarget = (State == ECheckBoxState::Checked);
	}
}

bool FVertexBrushSculptPropertiesDetails::IsFreezeTargetEnabled()
{
	return (TargetTool.IsValid()) ? TargetTool.Get()->SculptProperties->bFreezeTarget : false;
}









class FRecentAlphasProvider : public SToolInputAssetComboPanel::IRecentAssetsProvider
{
public:
	TArray<FAssetData> RecentAssets;

	virtual TArray<FAssetData> GetRecentAssetsList() override
	{
		return RecentAssets;
	}
	virtual void NotifyNewAsset(const FAssetData& NewAsset) override
	{
		if (NewAsset.GetAsset() == nullptr)
		{
			return;
		}
		for (int32 k = 0; k < RecentAssets.Num(); ++k)
		{
			if (RecentAssets[k] == NewAsset)
			{
				if (k == 0)
				{
					return;
				}
				RecentAssets.RemoveAt(k, EAllowShrinking::No);
			}
		}
		RecentAssets.Insert(NewAsset, 0);
		
		if (RecentAssets.Num() > 10)
		{
			RecentAssets.SetNum(10, EAllowShrinking::No);
		}
	}
};



TSharedRef<IDetailCustomization> FVertexBrushAlphaPropertiesDetails::MakeInstance()
{
	return MakeShareable(new FVertexBrushAlphaPropertiesDetails);
}

FVertexBrushAlphaPropertiesDetails::~FVertexBrushAlphaPropertiesDetails()
{
	if (TargetTool.IsValid())
	{
		TargetTool.Get()->OnDetailsPanelRequestRebuild.Remove(AlphaTextureUpdateHandle);
	}
}

void FVertexBrushAlphaPropertiesDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// TODO: move this to a subsystem or UObject CDO
	struct FRecentAlphasContainer
	{
		TSharedPtr<FRecentAlphasProvider> RecentAlphas;
	};
	static FRecentAlphasContainer RecentAlphasStatic;
	if (RecentAlphasStatic.RecentAlphas.IsValid() == false)
	{
		RecentAlphasStatic.RecentAlphas = MakeShared<FRecentAlphasProvider>();
	}
	RecentAlphasProvider = RecentAlphasStatic.RecentAlphas;

	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);
	UVertexBrushAlphaProperties* AlphaProperties = CastChecked<UVertexBrushAlphaProperties>(ObjectsBeingCustomized[0]);
	UMeshVertexSculptTool* Tool = AlphaProperties->Tool.Get();
	TargetTool = Tool;


	TSharedPtr<IPropertyHandle> AlphaHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushAlphaProperties, Alpha), UVertexBrushAlphaProperties::StaticClass());
	ensure(AlphaHandle->IsValidHandle());

	TSharedPtr<IPropertyHandle> RotationAngleHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushAlphaProperties, RotationAngle), UVertexBrushAlphaProperties::StaticClass());
	ensure(RotationAngleHandle->IsValidHandle());
	RotationAngleHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bRandomizeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushAlphaProperties, bRandomize), UVertexBrushAlphaProperties::StaticClass());
	ensure(bRandomizeHandle->IsValidHandle());
	bRandomizeHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> RandomRangeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushAlphaProperties, RandomRange), UVertexBrushAlphaProperties::StaticClass());
	ensure(RandomRangeHandle->IsValidHandle());
	RandomRangeHandle->MarkHiddenByCustomization();

	TSharedPtr<SDynamicNumericEntry::FDataSource> RotationAngleSource = SDynamicNumericEntry::MakeSimpleDataSource(
		RotationAngleHandle, TInterval<float>(-180.0f, 180.0f), TInterval<float>(-180.0f, 180.0f));
	TSharedPtr<SDynamicNumericEntry::FDataSource> RandomRangeSource = SDynamicNumericEntry::MakeSimpleDataSource(
		RandomRangeHandle, TInterval<float>(0.0f, 180.0f), TInterval<float>(0.0f, 180.0f));

	float ComboIconSize = 60;

	UModelingToolsModeCustomizationSettings* UISettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>();
	TArray<SToolInputAssetComboPanel::FNamedCollectionList> BrushAlphasLists;
	for (const FModelingModeAssetCollectionSet& AlphasCollectionSet : UISettings->BrushAlphaSets)
	{
		SToolInputAssetComboPanel::FNamedCollectionList CollectionSet;
		CollectionSet.Name = AlphasCollectionSet.Name;
		for (FCollectionReference CollectionRef : AlphasCollectionSet.Collections)
		{
			CollectionSet.Collections.Emplace(
				FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(),
				CollectionRef.CollectionName,
				ECollectionShareType::CST_Local);
		}
		BrushAlphasLists.Add(CollectionSet);
	}

	AlphaAssetPicker = SNew(SToolInputAssetComboPanel)
		.AssetClassType(UTexture2D::StaticClass())		// can infer from property...
		.Property(AlphaHandle)
		.ComboButtonTileSize(FVector2D(ComboIconSize, ComboIconSize))
		.FlyoutTileSize(FVector2D(80, 80))
		.FlyoutSize(FVector2D(1000, 600))
		.RecentAssetsProvider(RecentAlphasProvider)
		.CollectionSets(BrushAlphasLists);

	AlphaTextureUpdateHandle = TargetTool.Get()->OnDetailsPanelRequestRebuild.AddLambda([this]() {
		AlphaAssetPicker->RefreshThumbnailFromProperty();
	});

	DetailBuilder.EditDefaultProperty(AlphaHandle)->CustomWidget()
		.OverrideResetToDefault(FResetToDefaultOverride::Hide())
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0, ModelingUIConstants::DetailRowVertPadding))
			.AutoWidth()
			[
				SNew(SBox)
				.HeightOverride(ComboIconSize+14)
				[
					AlphaAssetPicker->AsShared()
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(ModelingUIConstants::MultiWidgetRowHorzPadding,ModelingUIConstants::DetailRowVertPadding,0,ModelingUIConstants::MultiWidgetRowHorzPadding))
			.FillWidth(1.0f)
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(FMargin(0))
					.AutoHeight()
					[
						MakeFixedWidthLabelSliderHBox(RotationAngleHandle, RotationAngleSource, FSculptToolsUIConstants::SculptShortLabelWidth)
					]

					+ SVerticalBox::Slot()
					.Padding(FMargin(0, ModelingUIConstants::DetailRowVertPadding))
					.AutoHeight()
					[
						MakeToggleSliderHBox(bRandomizeHandle, LOCTEXT("RandomizeLabel", "Rand"), RandomRangeSource, FSculptToolsUIConstants::SculptShortLabelWidth)
					]
			]
		];
	


}






#undef LOCTEXT_NAMESPACE
