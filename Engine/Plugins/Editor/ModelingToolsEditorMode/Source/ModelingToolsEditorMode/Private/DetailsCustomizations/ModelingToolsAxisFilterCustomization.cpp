// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/ModelingToolPropertyCustomizations.h"

#include "DetailWidgetRow.h"


#include "PropertySets/AxisFilterPropertyType.h"
#include "Widgets/SBoxPanel.h"


#define LOCTEXT_NAMESPACE "ModelingToolsAxisFilterCustomization"


TSharedRef<IPropertyTypeCustomization> FModelingToolsAxisFilterCustomization::MakeInstance()
{
	return MakeShareable(new FModelingToolsAxisFilterCustomization);
}

void FModelingToolsAxisFilterCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	const float XYZPadding = 10.f;

	TSharedPtr<IPropertyHandle> bAxisX = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FModelingToolsAxisFilter, bAxisX));
	bAxisX->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bAxisY = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FModelingToolsAxisFilter, bAxisY));
	bAxisY->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bAxisZ = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FModelingToolsAxisFilter, bAxisZ));
	bAxisZ->MarkHiddenByCustomization();

	HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
	.ValueContent()
		.MinDesiredWidth(125.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bAxisX->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bAxisX->CreatePropertyValueWidget()
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bAxisY->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bAxisY->CreatePropertyValueWidget()
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bAxisZ->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bAxisZ->CreatePropertyValueWidget()
				]
			]
		];
}

void FModelingToolsAxisFilterCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}


#undef LOCTEXT_NAMESPACE
