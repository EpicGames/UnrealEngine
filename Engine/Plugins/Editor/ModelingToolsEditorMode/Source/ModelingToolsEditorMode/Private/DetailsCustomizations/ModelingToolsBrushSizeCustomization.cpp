// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/ModelingToolsBrushSizeCustomization.h"

#include "DetailWidgetRow.h"
#include "ModelingToolsEditorModeStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "ModelingWidgets/ModelingCustomizationUtil.h"

#include "Sculpting/MeshSculptToolBase.h"
#include "Widgets/SBoxPanel.h"

using namespace UE::ModelingUI;

#define LOCTEXT_NAMESPACE "ModelingToolsBrushSizeCustomization"



TSharedRef<IPropertyTypeCustomization> FModelingToolsBrushSizeCustomization::MakeInstance()
{
	return MakeShareable(new FModelingToolsBrushSizeCustomization);
}

void FModelingToolsBrushSizeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> SizeType = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBrushToolRadius, SizeType));
	SizeType->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> AdaptiveSize = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBrushToolRadius, AdaptiveSize));
	AdaptiveSize->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> WorldRadius = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBrushToolRadius, WorldRadius));
	WorldRadius->MarkHiddenByCustomization();
	
	TSharedPtr<IPropertyHandle> EnablePressureSensitivity = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBrushToolRadius, bEnablePressureSensitivity));
	EnablePressureSensitivity->MarkHiddenByCustomization();
	
	TSharedPtr<IPropertyHandle> SupportPressureSensitivity = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBrushToolRadius, bToolSupportsPressureSensitivity));
	SupportPressureSensitivity->MarkHiddenByCustomization();

	auto GetCurrentSizeType = [SizeType]() -> EBrushToolSizeType
	{
		uint8 SizeTypeInt = 0;
		SizeType->GetValue(SizeTypeInt);
		return (SizeTypeInt == (uint8)EBrushToolSizeType::World) ? EBrushToolSizeType::World : EBrushToolSizeType::Adaptive;
	};
	auto GetSizeHandle = [SizeType, AdaptiveSize, WorldRadius]() -> TSharedPtr<IPropertyHandle>
	{
		uint8 SizeTypeInt = 0;
		SizeType->GetValue(SizeTypeInt);
		return (SizeTypeInt == (uint8)EBrushToolSizeType::World) ? WorldRadius : AdaptiveSize;
	};

	TSharedPtr<SDynamicNumericEntry::FDataSource> NumericSource = MakeShared<SDynamicNumericEntry::FDataSource>();
	NumericSource->SetValue = [GetSizeHandle](float NewSize, EPropertyValueSetFlags::Type Flags)
	{
		GetSizeHandle()->SetValue(NewSize, Flags);
	};
	NumericSource->GetValue = [GetSizeHandle]() -> float
	{
		float Size;
		GetSizeHandle()->GetValue(Size);
		return Size;
	};
	NumericSource->GetValueRange = [GetCurrentSizeType]() -> TInterval<float>
	{
		return (GetCurrentSizeType() == EBrushToolSizeType::Adaptive) ? TInterval<float>(0.0f, 10.0f) : TInterval<float>(0.01f, 50000.0f);
	};
	NumericSource->GetUIRange = [GetCurrentSizeType]() -> TInterval<float>
	{
		return (GetCurrentSizeType() == EBrushToolSizeType::Adaptive) ? TInterval<float>(0.0f, 1.0f) : TInterval<float>(1.0f, 1000.0f);
	};

	TSharedPtr<SHorizontalBox> Container;

	HeaderRow
	.OverrideResetToDefault(FResetToDefaultOverride::Hide())
	.WholeRowContent()
	[
		SAssignNew(Container, SHorizontalBox)
	];

	// name
	Container->AddSlot()
		.AutoWidth()
		.Padding(FMargin(0,4))
		[
			WrapInFixedWidthBox(AdaptiveSize->CreatePropertyNameWidget(), FSculptToolsUIConstants::SculptShortLabelWidth)
		];

	// slider
	Container->AddSlot()
		.Padding(ModelingUIConstants::LabelWidgetMinPadding, 0,0,0)
		.FillWidth(10.0f)
		[
			SNew(SDynamicNumericEntry)
			.Source(NumericSource)
		];

	// toggle
	Container->AddSlot()
		.AutoWidth()
		//.FillWidth(0.5f)
		.Padding(FMargin(ModelingUIConstants::MultiWidgetRowHorzPadding, ModelingUIConstants::DetailRowVertPadding, 0, ModelingUIConstants::DetailRowVertPadding))
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.WidthOverride(50)
			[
				SNew(SCheckBox)
					.Style(FAppStyle::Get(), "DetailsView.SectionButton")
					.Padding(FMargin(4, 2))
					.HAlign(HAlign_Center)
					.ToolTipText( LOCTEXT("WorldToggleTooltip", "Specify Brush Size in World Units") )
					.OnCheckStateChanged_Lambda([this, SizeType](ECheckBoxState State) {
						SizeType->SetValue( static_cast<uint8>( (State == ECheckBoxState::Checked) ? EBrushToolSizeType::World : EBrushToolSizeType::Adaptive ) );
					})
					.IsChecked_Lambda([this, SizeType]() {
						uint8 CurValueInt;
						SizeType->GetValue(CurValueInt);
						return (CurValueInt == static_cast<uint8>(EBrushToolSizeType::World)) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(FMargin(0))
						.AutoWidth()
						[
							SNew(STextBlock)
							.Justification(ETextJustify::Center)
							.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
							.Text( LOCTEXT("World", "World") )
						]
					]
			]
		];

	// pressure sensitivity toggle
	bool bAllowPressureSensitivity;
	SupportPressureSensitivity->GetValue(bAllowPressureSensitivity);
	if (bAllowPressureSensitivity)
	{
		Container->AddSlot()
			.AutoWidth()
			//.FillWidth(0.5f)
			.Padding(FMargin(ModelingUIConstants::MultiWidgetRowHorzPadding, ModelingUIConstants::DetailRowVertPadding, 0, ModelingUIConstants::DetailRowVertPadding))
			.HAlign(HAlign_Right)
			[
				SNew(SBox)
				.WidthOverride(30)
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "DetailsView.SectionButton")
					.Padding(FMargin(4, 2))
					.HAlign(HAlign_Center)
					.ToolTipText( LOCTEXT("PressureSensitivityToggle_Label", "Toggle Pressure Sensitivity for Brush Size"))
					.OnCheckStateChanged_Lambda([this, EnablePressureSensitivity](ECheckBoxState State) {
						
						EnablePressureSensitivity->SetValue( State == ECheckBoxState::Checked);
					})
					.IsChecked_Lambda([this, EnablePressureSensitivity]() {
						bool CurValue;
						EnablePressureSensitivity->GetValue(CurValue);
						return CurValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(FMargin(0))
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FModelingToolsEditorModeStyle::Get()->GetBrush("BrushIcons.PressureSensitivity.Small"))
						]
					]
			]
		];
	}

}


void FModelingToolsBrushSizeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}


#undef LOCTEXT_NAMESPACE
