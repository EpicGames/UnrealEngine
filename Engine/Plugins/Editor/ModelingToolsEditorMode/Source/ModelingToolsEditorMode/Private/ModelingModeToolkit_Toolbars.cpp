// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingToolsEditorModeToolkit.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ModelingToolsEditorMode.h"
#include "Framework/Commands/UICommandList.h"
#include "ModelingToolsManagerActions.h"
#include "ModelingToolsEditorModeSettings.h"
#include "ModelingToolsEditorModeStyle.h"

#include "ModelingSelectionInteraction.h"
#include "Selection/GeometrySelectionManager.h"

#include "ModelingComponentsSettings.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"

#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "STransformGizmoNumericalUIOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SComboButton.h"

// for showing toast notifications
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"

#include "ToolMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
// to toggle component instance selection

#define LOCTEXT_NAMESPACE "FModelingToolsEditorModeToolkit"


namespace UELocal
{

void MakeSubMenu_QuickSettings(FMenuBuilder& MenuBuilder)
{
	const FUIAction OpenModelingModeProjectSettings(
		FExecuteAction::CreateLambda([]
		{
			if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
			{
				SettingsModule->ShowViewer("Project", "Plugins", "ModelingMode");
			}
		}), FCanExecuteAction(), FIsActionChecked());
	MenuBuilder.AddMenuEntry(LOCTEXT("ModelingModeProjectSettings", "Modeling Mode (Project)"), 
		LOCTEXT("ModelingModeProjectSettings_Tooltip", "Jump to the Project Settings for Modeling Mode. Project Settings are Project-specific."),
		FSlateIcon(), OpenModelingModeProjectSettings, NAME_None, EUserInterfaceActionType::Button);

	const FUIAction OpenModelingModeEditorSettings(
		FExecuteAction::CreateLambda([]
		{
			if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
			{
				SettingsModule->ShowViewer("Editor", "Plugins", "ModelingMode");
			}
		}), FCanExecuteAction(), FIsActionChecked());
	MenuBuilder.AddMenuEntry(LOCTEXT("ModelingModeEditorSettings", "Modeling Mode (Editor)"), 
		LOCTEXT("ModelingModeEditorSettings_Tooltip", "Jump to the Editor Settings for Modeling Mode. Editor Settings apply across all Projects."),
		FSlateIcon(), OpenModelingModeEditorSettings, NAME_None, EUserInterfaceActionType::Button);

	const FUIAction OpenModelingToolsProjectSettings(
		FExecuteAction::CreateLambda([]
		{
			if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
			{
				SettingsModule->ShowViewer("Project", "Plugins", "Modeling Mode Tools");
			}
		}), FCanExecuteAction(), FIsActionChecked());
	MenuBuilder.AddMenuEntry(LOCTEXT("ModelingToolsProjectSettings", "Modeling Tools (Project)"), 
		LOCTEXT("ModelingToolsProjectSettings_Tooltip", "Jump to the Project Settings for Modeling Tools. Project Settings are Project-specific."),
		FSlateIcon(), OpenModelingToolsProjectSettings, NAME_None, EUserInterfaceActionType::Button);

}

void MakeSubMenu_GizmoVisibilityMode(FModelingToolsEditorModeToolkit* Toolkit, FMenuBuilder& MenuBuilder)
{
	UModelingToolsEditorModeSettings* Settings = GetMutableDefault<UModelingToolsEditorModeSettings>();
	UEditorInteractiveToolsContext* Context = Toolkit->GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode);

	// toggle for Combined/Separate Gizmo Mode
	const FUIAction GizmoMode_Combined(
		FExecuteAction::CreateLambda([Settings, Context]
		{
			Settings->bRespectLevelEditorGizmoMode = ! Settings->bRespectLevelEditorGizmoMode;
			Context->SetForceCombinedGizmoMode(Settings->bRespectLevelEditorGizmoMode == false);
			Settings->SaveConfig();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([Settings]()
		{
			return Settings->bRespectLevelEditorGizmoMode == false;
		}));
	MenuBuilder.AddMenuEntry(LOCTEXT("GizmoMode_Combined", "Combined Gizmo"), 
		LOCTEXT("GizmoMode_Combined_Tooltip", "Ignore Level Editor Gizmo Mode and always use a Combined Transform Gizmo in Modeling Tools"),
		FSlateIcon(), GizmoMode_Combined, NAME_None, EUserInterfaceActionType::ToggleButton);

	// toggle for Absolute Grid Snapping mode in World Coordinates
	const FUIAction GizmoMode_AbsoluteWorldSnap(
		FExecuteAction::CreateLambda([Settings, Context]
		{
			Context->SetAbsoluteWorldSnappingEnabled( ! Context->GetAbsoluteWorldSnappingEnabled() );
			Settings->bEnableAbsoluteWorldSnapping = Context->GetAbsoluteWorldSnappingEnabled();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([Context]()
		{
			return Context->GetAbsoluteWorldSnappingEnabled();
		}));
	MenuBuilder.AddMenuEntry(LOCTEXT("GizmoMode_AbsoluteGridSnap", "Absolute Grid Snapping"), 
		LOCTEXT("GizmoMode_AbsoluteGridSnap_Tooltip", "Snap translation/rotation to absolute grid coordinates in the world coordinate system, instead of relative to the initial position"),
		FSlateIcon(), GizmoMode_AbsoluteWorldSnap, NAME_None, EUserInterfaceActionType::ToggleButton);
	

	MenuBuilder.AddSubMenu(
		LOCTEXT("TransformPanelSubMenu", "Transform Panel"), LOCTEXT("TransformPanelSubMenu_ToolTip", "Configure the Gizmo Transform Panel."),
		FNewMenuDelegate::CreateLambda([Toolkit](FMenuBuilder& SubMenuBuilder) {
			TSharedPtr<STransformGizmoNumericalUIOverlay> NumericalUI = Toolkit->GetGizmoNumericalUIOverlayWidget();
			if (ensure(NumericalUI.IsValid()))
			{
				NumericalUI->MakeNumericalUISubMenu(SubMenuBuilder);
			}
		}));
}



void MakeSubMenu_SelectionSupport(FModelingToolsEditorModeToolkit* Toolkit, FMenuBuilder& MenuBuilder)
{
	UModelingToolsEditorModeSettings* Settings = GetMutableDefault<UModelingToolsEditorModeSettings>();

	// toggle for Combined/Separate Gizmo Mode
	const FUIAction EnableSelectionsToggle(
		FExecuteAction::CreateLambda([Toolkit, Settings]
		{
			Settings->SetMeshSelectionsEnabled(!Settings->GetMeshSelectionsEnabled());
			Settings->SaveConfig();
			Toolkit->NotifySelectionSystemEnabledStateModified();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([Settings]()
		{
			return Settings->GetMeshSelectionsEnabled();
		}));
	MenuBuilder.AddMenuEntry(LOCTEXT("MeshElementSelectionToggle", "Mesh Element Selection"), 
		LOCTEXT("MeshElementSelectionToggle_Tooltip", "Enable support for in-viewport Mesh Element Selection in Modeling Mode"),
		FSlateIcon(), EnableSelectionsToggle, NAME_None, EUserInterfaceActionType::ToggleButton);
}




void MakeSubMenu_ModeToggles(FMenuBuilder& MenuBuilder)
{
	// add toggle for Instance Selection cvar
	const FUIAction ToggleInstancesEverywhereAction(
		FExecuteAction::CreateLambda([]
		{
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TypedElements.EnableViewportSMInstanceSelection")); 
			int32 CurValue = CVar->GetInt();
			CVar->Set(CurValue == 0 ? 1 : 0);
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([]()
		{
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TypedElements.EnableViewportSMInstanceSelection"));
			return (CVar->GetInt() == 1);
		}));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToggleInstancesSelection", "Instance Selection"), 
		LOCTEXT("ToggleInstancesSelection_Tooltip", "Enable/Disable support for direct selection of InstancedStaticMeshComponent Instances (via TypedElements.EnableViewportInstanceSelection cvar)"),
		FSlateIcon(), ToggleInstancesEverywhereAction, NAME_None, EUserInterfaceActionType::ToggleButton);

	// add toggle for Volume Snapping cvar
	const FUIAction ToggleVolumeSnappingAction(
		FExecuteAction::CreateLambda([]
		{
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("modeling.EnableVolumeSnapping")); 
			bool CurValue = CVar->GetBool();
			CVar->Set(CurValue ? false : true);
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([]()
		{
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("modeling.EnableVolumeSnapping"));
			return (CVar->GetBool());
		}));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToggleVolumeSnapping", "Volume Snapping"), 
		LOCTEXT("ToggleVolumeSnapping_Tooltip", "Enable Vertex/Face Snapping and Ray-hits against Volumes in the Level. Note that if your level contains many overlapping large volumes, some Modeling functionality may not work correctly"),
		FSlateIcon(), ToggleVolumeSnappingAction, NAME_None, EUserInterfaceActionType::ToggleButton);


}


void MakeSubMenu_DefaultMeshObjectPhysicsSettings(FMenuBuilder& MenuBuilder)
{
	UModelingComponentsSettings* Settings = GetMutableDefault<UModelingComponentsSettings>();

	const FUIAction ToggleEnableCollisionAction = FUIAction(
		FExecuteAction::CreateLambda([Settings]
		{
			Settings->bEnableCollision = !Settings->bEnableCollision;
			Settings->SaveConfig();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([Settings]()
		{
			return Settings->bEnableCollision;
		}));

	MenuBuilder.AddMenuEntry(LOCTEXT("DefaultMeshObjectEnableCollision", "Enable Collision"), TAttribute<FText>(), FSlateIcon(), ToggleEnableCollisionAction, NAME_None, EUserInterfaceActionType::ToggleButton);

	MenuBuilder.BeginSection("CollisionTypeSection", LOCTEXT("CollisionTypeSection", "Collision Mode"));

	auto MakeCollisionTypeAction = [Settings](ECollisionTraceFlag FlagType)
	{
		return FUIAction(
			FExecuteAction::CreateLambda([Settings, FlagType]
			{
				Settings->CollisionMode = FlagType;
				Settings->SaveConfig();
			}),
			FCanExecuteAction::CreateLambda([Settings]() { return Settings->bEnableCollision; }),
			FIsActionChecked::CreateLambda([Settings, FlagType]()
			{
				return Settings->CollisionMode == FlagType;
			}));
	};
	const FUIAction CollisionMode_Default = MakeCollisionTypeAction(ECollisionTraceFlag::CTF_UseDefault);
	MenuBuilder.AddMenuEntry(LOCTEXT("DefaultMeshObject_CollisionDefault", "Default"), LOCTEXT("DefaultMeshObject_CollisionDefault_Tooltip", "Configure the new Mesh Object to have the Project Default collision settings (CTF_UseDefault)"),
		FSlateIcon(), CollisionMode_Default, NAME_None, EUserInterfaceActionType::ToggleButton);
	const FUIAction CollisionMode_Both = MakeCollisionTypeAction(ECollisionTraceFlag::CTF_UseSimpleAndComplex);
	MenuBuilder.AddMenuEntry(LOCTEXT("DefaultMeshObject_CollisionBoth", "Simple And Complex"), LOCTEXT("DefaultMeshObject_CollisionBoth_Tooltip", "Configure the new Mesh Object to have both Simple and Complex collision (CTF_UseSimpleAndComplex)"),
		FSlateIcon(), CollisionMode_Both, NAME_None, EUserInterfaceActionType::ToggleButton);
	const FUIAction CollisionMode_NoComplex = MakeCollisionTypeAction(ECollisionTraceFlag::CTF_UseSimpleAsComplex);
	MenuBuilder.AddMenuEntry(LOCTEXT("DefaultMeshObject_CollisionNoComplex", "Simple Only"), LOCTEXT("DefaultMeshObject_CollisionNoComplex_Tooltip", "Configure the new Mesh Object to have only Simple collision (CTF_UseSimpleAsComplex)"),
		FSlateIcon(), CollisionMode_NoComplex, NAME_None, EUserInterfaceActionType::ToggleButton);
	const FUIAction CollisionMode_ComplexOnly = MakeCollisionTypeAction(ECollisionTraceFlag::CTF_UseComplexAsSimple);
	MenuBuilder.AddMenuEntry(LOCTEXT("DefaultMeshObject_CollisionComplexOnly", "Complex Only"), LOCTEXT("DefaultMeshObject_CollisionComplexOnly_Tooltip", "Configure the new Mesh Object to have only Complex collision (CTF_UseComplexAsSimple)"),
		FSlateIcon(), CollisionMode_ComplexOnly, NAME_None, EUserInterfaceActionType::ToggleButton);

	MenuBuilder.EndSection();
}


void MakeSubMenu_DefaultMeshObjectType(FMenuBuilder& MenuBuilder)
{
	UModelingToolsEditorModeSettings* Settings = GetMutableDefault<UModelingToolsEditorModeSettings>();

	MenuBuilder.BeginSection("DefaultObjectTypeSection", LOCTEXT("DefaultObjectTypeSection", "Default Object Type"));

	auto ShowMeshObjectDefaultChangeToast = []()
	{
		FNotificationInfo Info(LOCTEXT("ChangeDefaultMeshObjectTypeToast", 
			"Changing the Default Mesh Object Type will take effect the next time the Editor is started"));
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	};


	MenuBuilder.AddMenuEntry(
		LOCTEXT("DefaultMeshObjectType_StaticMesh", "Static Mesh"), 
		TAttribute<FText>(), 
		FSlateIcon(), 
		FUIAction(
			FExecuteAction::CreateLambda([Settings, ShowMeshObjectDefaultChangeToast]
			{
				Settings->DefaultMeshObjectType = EModelingModeDefaultMeshObjectType::StaticMeshAsset;
				UCreateMeshObjectTypeProperties::DefaultObjectTypeIdentifier = UCreateMeshObjectTypeProperties::StaticMeshIdentifier;
				Settings->SaveConfig();
				ShowMeshObjectDefaultChangeToast();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([Settings]()
			{
				return Settings->DefaultMeshObjectType == EModelingModeDefaultMeshObjectType::StaticMeshAsset;
			})),
		NAME_None, EUserInterfaceActionType::ToggleButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DefaultMeshObjectType_DynamicMesh", "Dynamic Mesh Actor"),
		TAttribute<FText>(), 
		FSlateIcon(), 
		FUIAction(
			FExecuteAction::CreateLambda([Settings, ShowMeshObjectDefaultChangeToast]
			{
				Settings->DefaultMeshObjectType = EModelingModeDefaultMeshObjectType::DynamicMeshActor;
				UCreateMeshObjectTypeProperties::DefaultObjectTypeIdentifier = UCreateMeshObjectTypeProperties::DynamicMeshActorIdentifier;
				Settings->SaveConfig();
				ShowMeshObjectDefaultChangeToast();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([Settings]()
			{
				return Settings->DefaultMeshObjectType == EModelingModeDefaultMeshObjectType::DynamicMeshActor;
			})),
		NAME_None, EUserInterfaceActionType::ToggleButton );

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DefaultMeshObjectType_Volume", "Gameplay Volume"),
		TAttribute<FText>(), 
		FSlateIcon(), 
		FUIAction(
			FExecuteAction::CreateLambda([Settings, ShowMeshObjectDefaultChangeToast]
			{
				Settings->DefaultMeshObjectType = EModelingModeDefaultMeshObjectType::VolumeActor;
				UCreateMeshObjectTypeProperties::DefaultObjectTypeIdentifier = UCreateMeshObjectTypeProperties::VolumeIdentifier;
				Settings->SaveConfig();
				ShowMeshObjectDefaultChangeToast();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([Settings]()
			{
				return Settings->DefaultMeshObjectType == EModelingModeDefaultMeshObjectType::VolumeActor;
			})),
		NAME_None, EUserInterfaceActionType::ToggleButton);


	MenuBuilder.EndSection();

	MenuBuilder.AddMenuSeparator();

	MakeSubMenu_DefaultMeshObjectPhysicsSettings(MenuBuilder);
}

void MakeSubMenu_Selection_DragMode(FMenuBuilder& MenuBuilder)
{
	const FModelingToolsManagerCommands& Commands = FModelingToolsManagerCommands::Get();

	MenuBuilder.AddMenuEntry(
		Commands.SelectionDragMode_None,
		NAME_None,
		LOCTEXT("Selection_DragInput_None", "None"),
		LOCTEXT("Selection_DragInput_None_Tooltip", "No drag input"),
		FSlateIcon(),
		NAME_None
	);
	MenuBuilder.AddMenuEntry(
		Commands.SelectionDragMode_Path,
		NAME_None,
		LOCTEXT("Selection_DragInput_Path", "Path"),
		LOCTEXT("Selection_DragInput_Path_Tooltip", "Path Drag Input"),
		FSlateIcon(),
		NAME_None
	);
	
	// marquee mode is not functional yet, so this is disabled
	//MenuBuilder.AddMenuEntry(
	//	LOCTEXT("Selection_DragInput_Marquee", "Rectangle"), 
	//	LOCTEXT("Selection_DragInput_Marquee_Tooltip", "Rectangle Marquee"),
	//	FSlateIcon(), MakeDragModeOptionAction(EModelingSelectionInteraction_DragMode::RectangleMarqueeInteraction), NAME_None, EUserInterfaceActionType::ToggleButton);

}


void MakeSubMenu_Selection_MeshType(FMenuBuilder& MenuBuilder)
{
	const FModelingToolsManagerCommands& Commands = FModelingToolsManagerCommands::Get();

	MenuBuilder.AddMenuEntry(
		Commands.SelectionMeshTypes_Volumes,
		NAME_None,
		LOCTEXT("Selection_MeshTypes_Volumes", "Volumes"), 
		LOCTEXT("Selection_MeshTypes_Volumes_Tooltip", "Toggle whether Volume mesh elements can be selected in the Viewport"),
		FSlateIcon(),
		NAME_None
	);
	MenuBuilder.AddMenuEntry(
		Commands.SelectionMeshTypes_StaticMeshes,
		NAME_None,
		LOCTEXT("Selection_MeshTypes_StaticMesh", "Static Meshes"), 
		LOCTEXT("Selection_MeshTypes_StaticMesh_Tooltip", "Toggle whether Static Mesh mesh elements can be selected in the Viewport"),
		FSlateIcon(),
		NAME_None
	);
}
	
void MakeSubMenu_Selection_LocalFrameMode(FMenuBuilder& MenuBuilder)
{
	const FModelingToolsManagerCommands& Commands = FModelingToolsManagerCommands::Get();
	MenuBuilder.AddMenuEntry(
		Commands.SelectionLocalFrameMode_Geometry,
		NAME_None,
		LOCTEXT("Selection_LocalFrameMode_FromGeometry", "From Geometry"), 
		LOCTEXT("Selection_LocalFrameMode_FromGeometry_Tooltip", "Gizmo Orientation Based on Selected Geometry"),
		FSlateIcon(),
		NAME_None
	);
	MenuBuilder.AddMenuEntry(
		Commands.SelectionLocalFrameMode_Object,
		NAME_None,
		LOCTEXT("Selection_LocalFrameMode_FromObject", "From Object"), 
		LOCTEXT("Selection_LocalFrameMode_FromObject_Tooltip", "Gizmo Orientation Based on Object"),
		FSlateIcon(),
		NAME_None
	);
}


void MakeSubMenu_Selection_SelectionFilters(FMenuBuilder& MenuBuilder)
{
	const FModelingToolsManagerCommands& Commands = FModelingToolsManagerCommands::Get();
	MenuBuilder.AddMenuEntry(
		Commands.SelectionHitBackFaces,
		NAME_None,
		LOCTEXT("Selection_SelectionFilters_HitBackFaces", "Hit Back Faces"), 
		LOCTEXT("Selection_SelectionFilters_HitBackFaces_Tooltip", "Hit triangles that are facing away from the camera"),
		FSlateIcon(), NAME_None);
}

TSharedRef<SWidget> MakeMenu_SelectionConfigSettings(const FModelingToolsEditorModeToolkit* Toolkit)
{
	FMenuBuilder MenuBuilder(true, Toolkit->GetToolkitCommands());

	MenuBuilder.BeginSection("Section_DragMode", LOCTEXT("Section_DragMode", "Drag Mode"));
	MakeSubMenu_Selection_DragMode(MenuBuilder);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Section_MeshTypes", LOCTEXT("Section_MeshTypes", "Selectable Mesh Types"));
	MakeSubMenu_Selection_MeshType(MenuBuilder);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Section_LocalFrameMode", LOCTEXT("Section_LocalFrameMode", "Local Frame Mode"));
	MakeSubMenu_Selection_LocalFrameMode(MenuBuilder);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Section_SelectionFilters", LOCTEXT("Section_SelectionFilters", "Selection Filters"));
	MakeSubMenu_Selection_SelectionFilters(MenuBuilder);
	MenuBuilder.EndSection();

	TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();
	return MenuWidget;
}


} // end namespace UELocal


TSharedRef<SWidget> FModelingToolsEditorModeToolkit::MakeMenu_ModelingModeConfigSettings()
{
	using namespace UELocal;

	FMenuBuilder MenuBuilder(true, TSharedPtr<FUICommandList>());

	MenuBuilder.BeginSection("Section_Gizmo", LOCTEXT("Section_Gizmo", "Gizmo"));
	MakeSubMenu_GizmoVisibilityMode(this, MenuBuilder);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Section_SelectionSupport", LOCTEXT("Section_SelectionSupport", "Mesh Element Selection"));
	MakeSubMenu_SelectionSupport(this, MenuBuilder);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Section_MeshObjects", LOCTEXT("Section_MeshObjects", "New Mesh Objects"));
	MenuBuilder.AddSubMenu(
		LOCTEXT("MeshObjectTypeSubMenu", "New Mesh Settings"), LOCTEXT("MeshObjectTypeSubMenu_ToolTip", "Configure default settings for new Mesh Object Types"),
		FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder) {
			MakeSubMenu_DefaultMeshObjectType(SubMenuBuilder);
		}));
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Section_ModeToggles", LOCTEXT("Section_ModeToggles", "Mode Settings"));
	MakeSubMenu_ModeToggles(MenuBuilder);
	MenuBuilder.EndSection();

	// only show settings UI quick access in non-Restrictive mode
	const UModelingToolsEditorModeSettings* Settings = GetDefault<UModelingToolsEditorModeSettings>();
	if (!Settings->InRestrictiveMode())
	{
		MenuBuilder.BeginSection("Section_Settings", LOCTEXT("Section_Settings", "Quick Settings"));
		MenuBuilder.AddSubMenu(
			LOCTEXT("QuickSettingsSubMenu", "Jump To Settings"), LOCTEXT("QuickSettingsSubMenu_ToolTip", "Jump to sections of the Settings dialogs relevant to Modeling Mode"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder) {
				MakeSubMenu_QuickSettings(SubMenuBuilder);
				}));
		MenuBuilder.EndSection();
	}

	TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();
	return MenuWidget;
}

void FModelingToolsEditorModeToolkit::ExtendSecondaryModeToolbar(UToolMenu *InModeToolbarMenu)
{
	return;		// disable for now
#if 0
	/*
	 * Sanity check because the toolbar extension should happen after the ModeUILayer setup which is currently manual
	 * for standalone asset editors
	 */
	if( !ModeUILayer.IsValid() )
	{
		return;
	}

	UModelingToolsEditorModeSettings* ModelingModeSettings = GetMutableDefault<UModelingToolsEditorModeSettings>();
	const bool bEnableSelectionUI = ModelingModeSettings && ModelingModeSettings->GetMeshSelectionsEnabled();
	if ( !bEnableSelectionUI )
	{
		return;
	}
	
	FToolMenuSection& Section = InModeToolbarMenu->FindOrAddSection("SelectionPalette");

	/*
	 * NOTE: Any commands used in the SecondaryModeToolbar need to be present in the command list that lives in the
	 * ModeUILayer. Currently this is done automatically by parenting said command list to the toolkit's command list
	 */
	const FModelingToolsManagerCommands& Commands = FModelingToolsManagerCommands::Get();

	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.MeshSelectionModeAction_NoSelection));

	Section.AddSeparator(NAME_None);

	TSharedRef<SWidget> TrianglesTextWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(8.0f, 0.0f, 8.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("TriangleSelectionText", "Triangles"))
		];

	Section.AddEntry(FToolMenuEntry::InitWidget("TrianglesTextWidget", TrianglesTextWidget, FText()));
	
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.MeshSelectionModeAction_MeshTriangles));
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.MeshSelectionModeAction_MeshEdges));
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.MeshSelectionModeAction_MeshVertices));

	Section.AddSeparator(NAME_None);

	TSharedRef<SWidget> PolyGroupsTextWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(8.0f, 0.0f, 8.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("PolyGroupsSelectionText", "PolyGroups"))
		];

	Section.AddEntry(FToolMenuEntry::InitWidget("PolyGroupsTextWidget", PolyGroupsTextWidget, FText()));
	
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.MeshSelectionModeAction_GroupFaces));
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.MeshSelectionModeAction_GroupEdges));
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.MeshSelectionModeAction_GroupCorners));

	Section.AddSeparator(NAME_None);

	TSharedRef<SWidget> SelectionEditTextWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(8.0f, 0.0f, 8.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SelectionEditText", "Selection Edit"))
		];

	Section.AddEntry(FToolMenuEntry::InitWidget("SelectionEditTextWidget", SelectionEditTextWidget, FText()));

	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.BeginSelectionAction_SelectAll));
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.BeginSelectionAction_Invert));
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.BeginSelectionAction_ExpandToConnected));
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.BeginSelectionAction_InvertConnected));
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.BeginSelectionAction_Expand));
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.BeginSelectionAction_Contract));
	
	Section.AddSeparator(NAME_None);

	TSharedRef<SWidget> DefaultSettingsWidget = SNew(SComboButton)
		.HasDownArrow(false)
		.MenuPlacement(EMenuPlacement::MenuPlacement_MenuRight)
		.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
		.OnGetMenuContent(FOnGetContent::CreateLambda([this]()
		{
			return UELocal::MakeMenu_SelectionConfigSettings(this);
		}))
		.ContentPadding(FMargin(3.0f, 1.0f))
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FModelingToolsEditorModeStyle::Get()->GetBrush("ModelingMode.DefaultSettings"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];

	Section.AddEntry(FToolMenuEntry::InitWidget("DefaultSettings", DefaultSettingsWidget, FText()));
	
	Section.AddSeparator(NAME_None);
#endif
}

#undef LOCTEXT_NAMESPACE
