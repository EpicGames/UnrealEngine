// Copyright Epic Games, Inc. All Rights Reserved.


#include "ModelingSelectionInteraction.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "InteractiveToolsContext.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "Selection/GeometrySelectionManager.h"
#include "InteractiveGizmoManager.h"
#include "ToolSceneQueriesUtil.h"
#include "InteractiveToolManager.h"
#include "SceneQueries/SceneSnappingManager.h"

#include "BaseBehaviors/SingleClickOrDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseBehaviors/KeyAsModifierInputBehavior.h"

#include "Mechanics/RectangleMarqueeMechanic.h"
#include "Mechanics/DragAlignmentMechanic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelingSelectionInteraction)

using namespace UE::Geometry;

void UModelingSelectionInteraction::Initialize(
	TObjectPtr<UGeometrySelectionManager> SelectionManagerIn,
	TUniqueFunction<bool()> CanChangeSelectionCallbackIn,
	TUniqueFunction<bool(const FInputDeviceRay&)> ExternalHitCaptureCallbackIn)
{
	SelectionManager = SelectionManagerIn;

	CanChangeSelectionCallback = MoveTemp(CanChangeSelectionCallbackIn);
	ExternalHitCaptureCallback = MoveTemp(ExternalHitCaptureCallbackIn);

	// set up rectangle marquee interaction
	RectangleMarqueeInteraction = NewObject<URectangleMarqueeInteraction>(this);
	RectangleMarqueeInteraction->OnDragRectangleFinished.AddUObject(this, &UModelingSelectionInteraction::OnMarqueeRectangleFinished);

	// set up path selection interaction
	PathSelectionInteraction = NewObject<UPathSelectionInteraction>(this);
	PathSelectionInteraction->Setup(this);

	// create click behavior and set ourselves as click target
	ClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	ClickOrDragBehavior->Modifiers.RegisterModifier(AddToSelectionModifier, FInputDeviceState::IsShiftKeyDown);
	ClickOrDragBehavior->Modifiers.RegisterModifier(ToggleSelectionModifier, FInputDeviceState::IsCtrlKeyDown);
	ClickOrDragBehavior->bBeginDragIfClickTargetNotHit = false;
	ClickOrDragBehavior->Initialize(this, PathSelectionInteraction);

	HoverBehavior = NewObject<UMouseHoverBehavior>(this);
	HoverBehavior->Initialize(this);

	BehaviorSet = NewObject<UInputBehaviorSet>();
	BehaviorSet->Add(ClickOrDragBehavior, this);
	BehaviorSet->Add(HoverBehavior, this);

	// configure drag mode of ClickOrDragBehavior
	UpdateActiveDragMode();

	TransformProxy = NewObject<UTransformProxy>(this);
	TransformGizmo = UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo( 
		SelectionManager->GetToolsContext()->GizmoManager, ETransformGizmoSubElements::FullTranslateRotateScale,
		this );
	TransformGizmo->SetIsNonUniformScaleAllowedFunction([]() {return true;});
	TransformGizmo->SetActiveTarget(TransformProxy, SelectionManager->GetToolsContext()->GizmoManager);
	TransformGizmo->SetVisibility(false);

	DragAlignmentInteraction = NewObject<UDragAlignmentInteraction>(this);
	DragAlignmentInteraction->Setup( USceneSnappingManager::Find(SelectionManager->GetToolsContext()->GizmoManager) );

	DragAlignmentToggleBehavior = NewObject<UKeyAsModifierInputBehavior>(this);
	DragAlignmentInteraction->RegisterAsBehaviorTarget(DragAlignmentToggleBehavior);
	BehaviorSet->Add(DragAlignmentToggleBehavior, this);
	DragAlignmentInteraction->AddToGizmo(TransformGizmo);

	// listen for change events on transform proxy
	TransformProxy->OnTransformChanged.AddUObject(this, &UModelingSelectionInteraction::OnGizmoTransformChanged);
	TransformProxy->OnBeginTransformEdit.AddUObject(this, &UModelingSelectionInteraction::OnBeginGizmoTransform);
	TransformProxy->OnEndTransformEdit.AddUObject(this, &UModelingSelectionInteraction::OnEndGizmoTransform);

	// listen for selection changes to update gizmo
	OnSelectionModifiedDelegate = SelectionManager->OnSelectionModified.AddUObject(this, &UModelingSelectionInteraction::OnSelectionManager_SelectionModified);
}

void UModelingSelectionInteraction::Shutdown()
{
	if (IsValid(SelectionManager))
	{
		SelectionManager->OnSelectionModified.Remove(OnSelectionModifiedDelegate);
	}

	if (TransformGizmo)
	{
		SelectionManager->GetToolsContext()->GizmoManager->DestroyGizmo(TransformGizmo);
		TransformGizmo = nullptr;
	}
}

void UModelingSelectionInteraction::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (DragAlignmentInteraction)
	{
		DragAlignmentInteraction->Render(RenderAPI);
	}
}

void UModelingSelectionInteraction::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (RectangleMarqueeInteraction)
	{
		RectangleMarqueeInteraction->DrawHUD(Canvas, RenderAPI);
	}
}

void UModelingSelectionInteraction::SetActiveDragMode(EModelingSelectionInteraction_DragMode NewMode)
{
	if (ActiveDragMode != NewMode)
	{
		ActiveDragMode = NewMode;
		UpdateActiveDragMode();
	}
}

void UModelingSelectionInteraction::SetLocalFrameMode(EModelingSelectionInteraction_LocalFrameMode NewLocalFrameMode)
{
	if (LocalFrameMode != NewLocalFrameMode)
	{
		LocalFrameMode = NewLocalFrameMode;
		UpdateGizmoOnSelectionChange();
	}
}

void UModelingSelectionInteraction::UpdateActiveDragMode()
{
	if (ActiveDragMode == EModelingSelectionInteraction_DragMode::PathInteraction)
	{
		ClickOrDragBehavior->SetDragTarget(PathSelectionInteraction);
	}
	else if (ActiveDragMode == EModelingSelectionInteraction_DragMode::RectangleMarqueeInteraction)
	{
		ClickOrDragBehavior->SetDragTarget(RectangleMarqueeInteraction);
	}
	else
	{
		ClickOrDragBehavior->SetDragTarget(nullptr);
	}
}


void UModelingSelectionInteraction::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	// update modifier state flags
	if (ModifierID == AddToSelectionModifier)
	{
		bAddToSelectionEnabled = bIsOn;
	}
	else if (ModifierID == ToggleSelectionModifier)
	{
		bToggleSelectionEnabled = bIsOn;
	}
}


void UModelingSelectionInteraction::ComputeSceneHits(const FInputDeviceRay& ClickPos,
	bool& bHitActiveObjects, FInputRayHit& ActiveObjectHit,
	bool& bHitInactiveObjectFirst, FInputRayHit& InactiveObjectHit)
{
	ActiveObjectHit = FInputRayHit();
	bHitActiveObjects = SelectionManager->RayHitTest(ClickPos.WorldRay, ActiveObjectHit);

	// We want to filter out hits against nearer objects. This is...tricky.
	// TODO: we probably should not actually do this here. Modeling Mode probably needs to 
	// have a behavior that mimics editor selection, that would have just received the closer
	// FInputRayHit and taken the click, instead of this function...
	bHitInactiveObjectFirst = false;
	InactiveObjectHit = FInputRayHit();
	if (bHitActiveObjects)
	{
		TArray<const UPrimitiveComponent*> IgnoreComponents;
		if (ActiveObjectHit.HitObject.IsValid())
		{
			if (UPrimitiveComponent* SelectionComponent = Cast<UPrimitiveComponent>(ActiveObjectHit.HitObject.Get()) )
			{
				IgnoreComponents.Add(SelectionComponent);
			}
		}
		FHitResult HitResultOut;
		if (ToolSceneQueriesUtil::FindNearestVisibleObjectHit(USceneSnappingManager::Find(SelectionManager->GetToolsContext()->ToolManager), 
			HitResultOut, ClickPos.WorldRay, &IgnoreComponents, nullptr))
		{
			if (HitResultOut.Distance < ActiveObjectHit.HitDepth)
			{
				bHitInactiveObjectFirst = true;
				InactiveObjectHit.HitDepth = HitResultOut.Distance;
				InactiveObjectHit.bHit = true;
			}
		}
	}
}


FInputRayHit UModelingSelectionInteraction::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	// ignore hits in these cases
	if (  (SelectionManager->GetMeshTopologyMode() == UGeometrySelectionManager::EMeshTopologyMode::None) 
		|| (CanChangeSelectionCallback() == false)
		|| ExternalHitCaptureCallback(ClickPos) )
	{
		return FInputRayHit();
	}

	bool bHitActiveObjects, bHitInactiveObjectFirst;
	FInputRayHit ActiveObjectHit, InactiveObjectHit;
	ComputeSceneHits(ClickPos, bHitActiveObjects, ActiveObjectHit, bHitInactiveObjectFirst, InactiveObjectHit);

	// Note: currently this flow will produce somewhat undesirable behavior, in that if 
	// some other object not in the selection target(s) is clicked, the selection will be 
	// cleared but that object will have to be clicked again to select it. We cannot currently 
	// clear the mesh-selection and change the actor-selection inside a single transaction in a
	// single click. This is because we are relying on the Editor to do the actor-selection change.
	// So if we were to (eg) ClearSelection() and return no-hit here, the ClearSelection() call
	// would emit a transaction, and then the actor-selection-change would emit a second transaction.
	// 
	// Behavior in many DCCS is that they can (1) clear active mesh-selection, (2) select new target
	// object, and (3) do new mesh-selection in a single click. This will require quite a bit of effort
	// to achieve in the future. 
	//
	// So for now we set a bClearSelectionOnClicked flag and consume the click. This means the 1-2-3
	// flow above requires 3 separate clicks :( 


	bClearSelectionOnClicked = false;
	if (SelectionManager->HasSelection())
	{
		if (bHitInactiveObjectFirst)
		{
			//bClearSelectionOnClicked = true;
			//return InactiveObjectHit;
			return FInputRayHit();
		}
		else if (bHitActiveObjects)
		{
			return ActiveObjectHit;
		}
		else
		{
			// if we have active selection we still want to capture this hit so that we can clear in OnClicked
			//bClearSelectionOnClicked = true;
			//return FInputRayHit(TNumericLimits<double>::Max());
			return FInputRayHit();
		}
	}
	else
	{
		if (bHitActiveObjects && bHitInactiveObjectFirst == false)
		{
			return ActiveObjectHit;
		}
	}

	return FInputRayHit();
}



void UModelingSelectionInteraction::OnClicked(const FInputDeviceRay& ClickPos)
{
	// this flag is set in IsHitByClick test
	if (bClearSelectionOnClicked)
	{
		SelectionManager->ClearSelection();
		bClearSelectionOnClicked = false;
		return;
	}

	FGeometrySelectionUpdateConfig UpdateConfig = GetActiveSelectionUpdateConfig();

	FGeometrySelectionUpdateResult Result;
	SelectionManager->UpdateSelectionViaRaycast(
		ClickPos.WorldRay,
		UpdateConfig,
		Result);
}


FGeometrySelectionUpdateConfig UModelingSelectionInteraction::GetActiveSelectionUpdateConfig() const
{
	FGeometrySelectionUpdateConfig UpdateConfig;
	UpdateConfig.ChangeType = EGeometrySelectionChangeType::Replace;
	if (bAddToSelectionEnabled)
	{
		UpdateConfig.ChangeType = EGeometrySelectionChangeType::Add;
	}
	else if (bToggleSelectionEnabled)
	{
		UpdateConfig.ChangeType = EGeometrySelectionChangeType::Remove;
	}
	return UpdateConfig;
}



FInputRayHit UModelingSelectionInteraction::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	if ((SelectionManager->GetMeshTopologyMode() == UGeometrySelectionManager::EMeshTopologyMode::None)
		|| (CanChangeSelectionCallback() == false)
		|| ExternalHitCaptureCallback(PressPos))
	{
		return FInputRayHit();
	}

	bool bHitActiveObjects, bHitInactiveObjectFirst;
	FInputRayHit ActiveObjectHit, InactiveObjectHit;
	ComputeSceneHits(PressPos, bHitActiveObjects, ActiveObjectHit, bHitInactiveObjectFirst, InactiveObjectHit);

	if (bHitActiveObjects && bHitInactiveObjectFirst == false)
	{
		return ActiveObjectHit;
	}
	return FInputRayHit();
}

void UModelingSelectionInteraction::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	SelectionManager->UpdateSelectionPreviewViaRaycast(DevicePos.WorldRay);
}

bool UModelingSelectionInteraction::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	return SelectionManager->UpdateSelectionPreviewViaRaycast(DevicePos.WorldRay);
}

void UModelingSelectionInteraction::OnEndHover()
{
	SelectionManager->ClearSelectionPreview();
}




void UModelingSelectionInteraction::UpdateGizmoOnSelectionChange()
{
	if (SelectionManager->HasSelection() == false)
	{
		TransformGizmo->SetVisibility(false);
	}
	else
	{
		TransformGizmo->SetVisibility(true);

		FFrame3d SelectionFrame;
		if (LocalFrameMode == EModelingSelectionInteraction_LocalFrameMode::FromGeometry)
		{
			SelectionManager->GetSelectionWorldFrame(SelectionFrame);
		}
		else
		{
			SelectionManager->GetTargetWorldFrame(SelectionFrame);
		}
		TransformGizmo->ReinitializeGizmoTransform( SelectionFrame.ToFTransform() );
	}
}


void UModelingSelectionInteraction::OnSelectionManager_SelectionModified()
{
	UpdateGizmoOnSelectionChange();
}



void UModelingSelectionInteraction::OnBeginGizmoTransform(UTransformProxy* Proxy)
{
	FTransform Transform = Proxy->GetTransform();
	InitialGizmoFrame = FFrame3d(Transform);
	InitialGizmoScale = FVector3d(Transform.GetScale3D());

	LastTranslationDelta = FVector3d::Zero();
	LastRotateDelta = FVector4d::Zero();
	LastScaleDelta = FVector3d::Zero();

	bInActiveTransform = SelectionManager->BeginTransformation();

	OnTransformBegin.Broadcast();
}

void UModelingSelectionInteraction::OnEndGizmoTransform(UTransformProxy* Proxy)
{
	if (bInActiveTransform)
	{
		// If we have an update we haven't applied, apply it now. This is particularly important if the
		// user transformed the gizmo by typing a new value into the gizmo UI, in which case there will
		// not have been a chance to do ApplyPendingTransformInteractions() between the update and end calls.
		if (bGizmoUpdatePending)
		{
			ApplyPendingTransformInteractions();
		}

		// Clear the active-transform flag before calling EndTransformation. EndTransformation can synchronously
		// re-enter this handler if the underlying commit (e.g. UStaticMesh::PostEditChange) pumps messages and
		// the resulting focus loss is routed through UInputRouter::ForceTerminateAll back through the gizmo.
		// Clearing the flag first ensures the reentrant call does not double-broadcast OnTransformEnd.
		//
		// Note: this must come after ApplyPendingTransformInteractions, which guards its own work on bInActiveTransform being true.
		bInActiveTransform = false;

		SelectionManager->EndTransformation();

		// reset the transient scaling being stored by the gizmo, otherwise it will be re-applied in the next transformation
		TransformGizmo->SetNewChildScale(FVector::One());

		OnTransformEnd.Broadcast();
	}
}

void UModelingSelectionInteraction::OnGizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	if (bInActiveTransform)
	{
		LastUpdateGizmoFrame = FFrame3d(Transform);
		LastUpdateGizmoScale = FVector3d(Transform.GetScale3D());
		bGizmoUpdatePending = true;
	}
}


void UModelingSelectionInteraction::ApplyPendingTransformInteractions()
{
	if (bInActiveTransform == false || bGizmoUpdatePending == false)
	{
		return;
	}

	FFrame3d CurFrame = LastUpdateGizmoFrame;
	FVector3d CurScale = LastUpdateGizmoScale;
	FVector3d TranslationDelta = CurFrame.Origin - InitialGizmoFrame.Origin;
	FQuaterniond RotateDelta = CurFrame.Rotation - InitialGizmoFrame.Rotation;
	FVector3d CurScaleDelta = CurScale - InitialGizmoScale;

	bool bTransformInWorldFrame = TransformGizmo->CurrentCoordinateSystem == EToolContextCoordinateSystem::World;

	// if any of the deltas have deviated from the last delta, forward the transformation change on to targets
	if ( (TranslationDelta - LastTranslationDelta).SizeSquared() > FMathf::ZeroTolerance
		|| ((FVector4d)RotateDelta - LastRotateDelta).SizeSquared() > FMathf::ZeroTolerance
		|| (CurScaleDelta- LastScaleDelta).SizeSquared() > FMathf::ZeroTolerance)
	{
		LastTranslationDelta = TranslationDelta;
		LastRotateDelta = (FVector4d)RotateDelta;
		LastScaleDelta = CurScaleDelta;

		FQuat RotationDelta = FQuat(CurFrame.Rotation * InitialGizmoFrame.Rotation.Inverse());

		SelectionManager->UpdateTransformation(InitialGizmoFrame.ToFTransform(), TranslationDelta, RotationDelta, CurScaleDelta, bTransformInWorldFrame);
	}

	bGizmoUpdatePending = false;
}



void UModelingSelectionInteraction::OnMarqueeRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled)
{

}




void UPathSelectionInteraction::Setup(UModelingSelectionInteraction* SelectionInteractionIn)
{

	SelectionInteraction = SelectionInteractionIn;
}


FInputRayHit UPathSelectionInteraction::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	if (SelectionInteraction->GetSelectionManager()->CanBeginTrackedSelectionChange())
	{
		return SelectionInteraction->IsHitByClick(PressPos);
	}
	return FInputRayHit();
}

void UPathSelectionInteraction::OnClickPress(const FInputDeviceRay& PressPos)
{
	FGeometrySelectionUpdateConfig UpdateConfig = SelectionInteraction->GetActiveSelectionUpdateConfig();
	UGeometrySelectionManager* SelectionManager = SelectionInteraction->GetSelectionManager();

	bool bInitialClear = false;
	if (UpdateConfig.ChangeType == EGeometrySelectionChangeType::Replace)
	{
		bInitialClear = true;
		UpdateConfig.ChangeType = EGeometrySelectionChangeType::Add;
	}

	if (SelectionManager->BeginTrackedSelectionChange(UpdateConfig, bInitialClear))
	{
		FGeometrySelectionUpdateResult Result;
		SelectionManager->AccumulateSelectionUpdate_Raycast(PressPos.WorldRay, Result);
	}
}

void UPathSelectionInteraction::OnClickDrag(const FInputDeviceRay& DragPos)
{
	UGeometrySelectionManager* SelectionManager = SelectionInteraction->GetSelectionManager();
	if (SelectionManager->IsInTrackedSelectionChange())
	{
		FGeometrySelectionUpdateResult Result;
		SelectionManager->AccumulateSelectionUpdate_Raycast(DragPos.WorldRay, Result);		
	}
}

void UPathSelectionInteraction::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	UGeometrySelectionManager* SelectionManager = SelectionInteraction->GetSelectionManager();
	if (SelectionManager->IsInTrackedSelectionChange())
	{
		SelectionManager->EndTrackedSelectionChange();
	}
}

void UPathSelectionInteraction::OnTerminateDragSequence()
{
	UGeometrySelectionManager* SelectionManager = SelectionInteraction->GetSelectionManager();
	if (SelectionManager->IsInTrackedSelectionChange())
	{
		SelectionManager->EndTrackedSelectionChange();
	}
}
