// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if ENABLE_STYLUS_SUPPORT

#include "CoreMinimal.h"
#include "StylusInput.h"
#include "StylusInputPacket.h"
#include "TickableEditorObject.h"

#include "BaseTools/MeshSurfacePointTool.h"
#include "Widgets/SWidget.h"

namespace UE::Modeling
{

/**
 * FStylusInputHandler registers itself as a listener for stylus events and implements the
 * IToolStylusStateProviderAPI interface, which allows MeshSurfacePointTool implementations
 * to query for pen pressure.
 *
 * This is kind of a hack. Unfortunately, the current Stylus module is a Plugin so it
 * cannot be used in the base ToolsFramework directly. So, we need this in the Mode as a
 * workaround.
 */
class FStylusInputHandler final : public UE::StylusInput::IStylusInputEventHandler, public IToolStylusStateProviderAPI
{
public:
	FStylusInputHandler();
	virtual ~FStylusInputHandler() override;

	/**
	 * Registers the window containing the given Widget for Stylus input handling.
	 * Registered windows persist for the lifetime of the StylusInputHandler.
	 * 
	 * @param Widget The widget whose window is to be registered.
	 * @return True if the window was registered, false if the window was invalid or was previously registered.
	 */
	bool RegisterWindow(const TSharedRef<SWidget>& Widget);

	
	// IStylusInputEventHandler implementation
	virtual FString GetName() override;
	virtual void OnPacket(const UE::StylusInput::FStylusInputPacket& Packet, UE::StylusInput::IStylusInputInstance* Instance) override;

	
	// IToolStylusStateProviderAPI implementation
	virtual float GetCurrentPressure() const override
	{
		return ActivePressure;
	}

private:
	void ProcessPacket(const UE::StylusInput::FStylusInputPacket& Packet, UE::StylusInput::IStylusInputInstance* Instance);
	const UE::StylusInput::IStylusInputTabletContext* GetTabletContext(UE::StylusInput::IStylusInputInstance* Instance, uint32 TabletContextID);

	TMap<TSharedPtr<SWindow>, UE::StylusInput::IStylusInputInstance*> StylusInputInstances;
	TMap<uint32, TSharedPtr<UE::StylusInput::IStylusInputTabletContext>> TabletContexts;

	float ActivePressure = 1.0f;
};

}

#endif // ENABLE_STYLUS_SUPPORT
