// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingToolsActions.h"
#include "Styling/AppStyle.h"
#include "DynamicMeshSculptTool.h"
#include "EditMeshMaterialsTool.h"
#include "MeshVertexSculptTool.h"
#include "MeshGroupPaintTool.h"
#include "MeshVertexPaintTool.h"
#include "MeshAttributePaintTool.h"
#include "MeshInspectorTool.h"
#include "CubeGridTool.h"
#include "DrawPolygonTool.h"
#include "EditMeshPolygonsTool.h"
#include "ShapeSprayTool.h"
#include "MeshSpaceDeformerTool.h"
#include "Tools/StandardToolModeCommands.h"
#include "TransformMeshesTool.h"
#include "PlaneCutTool.h"
#include "EditMeshPolygonsTool.h"
#include "DrawAndRevolveTool.h"

#define LOCTEXT_NAMESPACE "ModelingToolsCommands"



FModelingModeActionCommands::FModelingModeActionCommands() :
	TCommands<FModelingModeActionCommands>(
		"ModelingModeCommands", // Context name for fast lookup
		NSLOCTEXT("Contexts", "ModelingModeCommands", "Modeling Mode"), // Localized context name for displaying
		NAME_None, // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
{
}


void FModelingModeActionCommands::RegisterCommands()
{
	UI_COMMAND(FocusViewCommand, "Focus View at Cursor", "Focuses the camera at the scene hit location under the cursor", EUserInterfaceActionType::None, FInputChord(EKeys::C));
	UI_COMMAND(ToggleSelectionLockStateCommand, "Toggle Selection Lock State", "Toggles the Locked/Unlocked state of the active Selection Target", EUserInterfaceActionType::None, FInputChord(EKeys::U));
}


void FModelingModeActionCommands::RegisterCommandBindings(TSharedPtr<FUICommandList> UICommandList, TFunction<void(EModelingModeActionCommands)> OnCommandExecuted)
{
	const FModelingModeActionCommands& Commands = FModelingModeActionCommands::Get();

	UICommandList->MapAction(
		Commands.FocusViewCommand,
		FExecuteAction::CreateLambda([OnCommandExecuted]() { OnCommandExecuted(EModelingModeActionCommands::FocusViewToCursor); }));

	UICommandList->MapAction(
		Commands.ToggleSelectionLockStateCommand,
		FExecuteAction::CreateLambda([OnCommandExecuted]() { OnCommandExecuted(EModelingModeActionCommands::ToggleSelectionLockState); }));
}

void FModelingModeActionCommands::UnRegisterCommandBindings(TSharedPtr<FUICommandList> UICommandList)
{
	const FModelingModeActionCommands& Commands = FModelingModeActionCommands::Get();
	UICommandList->UnmapAction(Commands.FocusViewCommand);
	UICommandList->UnmapAction(Commands.ToggleSelectionLockStateCommand);
}



FModelingToolActionCommands::FModelingToolActionCommands() : 
	TInteractiveToolCommands<FModelingToolActionCommands>(
		"ModelingToolsEditMode", // Context name for fast lookup
		NSLOCTEXT("Contexts", "ModelingToolsEditMode", "Modeling Tools - Shared Shortcuts"), // Localized context name for displaying
		NAME_None, // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
{
}


void FModelingToolActionCommands::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs)
{
	ToolCDOs.Add(GetMutableDefault<UMeshInspectorTool>());
	//ToolCDOs.Add(GetMutableDefault<UDrawPolygonTool>());
	//ToolCDOs.Add(GetMutableDefault<UEditMeshPolygonsTool>());
	ToolCDOs.Add(GetMutableDefault<UMeshSpaceDeformerTool>());
	ToolCDOs.Add(GetMutableDefault<UShapeSprayTool>());
	//ToolCDOs.Add(GetMutableDefault<UPlaneCutTool>());
}



void FModelingToolActionCommands::RegisterAllToolActions()
{
	FModelingToolActionCommands::Register();
	FSculptToolActionCommands::Register();
	FVertexSculptToolActionCommands::Register();
	FMeshGroupPaintToolActionCommands::Register();
	FMeshVertexPaintToolActionCommands::Register();
	FMeshAttributePaintToolActionCommands::Register();
	FDrawPolygonToolActionCommands::Register();
	FTransformToolActionCommands::Register();
	FMeshSelectionToolActionCommands::Register();
	FEditMeshMaterialsToolActionCommands::Register();
	FMeshPlaneCutToolActionCommands::Register();
	FCubeGridToolActionCommands::Register();
	FEditMeshPolygonsToolActionCommands::Register();
	FDrawAndRevolveToolActionCommands::Register();
}

void FModelingToolActionCommands::UnregisterAllToolActions()
{
	FModelingToolActionCommands::Unregister();
	FSculptToolActionCommands::Unregister();
	FVertexSculptToolActionCommands::Unregister();
	FMeshGroupPaintToolActionCommands::Unregister();
	FMeshVertexPaintToolActionCommands::Unregister();
	FMeshAttributePaintToolActionCommands::Unregister();
	FDrawPolygonToolActionCommands::Unregister();
	FTransformToolActionCommands::Unregister();
	FMeshSelectionToolActionCommands::Unregister();
	FEditMeshMaterialsToolActionCommands::Unregister();
	FMeshPlaneCutToolActionCommands::Unregister();
	FCubeGridToolActionCommands::Unregister();
	FEditMeshPolygonsToolActionCommands::Unregister();
	FDrawAndRevolveToolActionCommands::Unregister();
}



void FModelingToolActionCommands::UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind)
{
#define UPDATE_BINDING(CommandsType)  if (!bUnbind) CommandsType::Get().BindCommandsForCurrentTool(UICommandList, Tool); else CommandsType::Get().UnbindActiveCommands(UICommandList);


	if (ExactCast<UTransformMeshesTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FTransformToolActionCommands);
	}
	else if (ExactCast<UDynamicMeshSculptTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FSculptToolActionCommands);
	}
	else if (ExactCast<UMeshVertexSculptTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FVertexSculptToolActionCommands);
	}
	else if (ExactCast<UMeshGroupPaintTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FMeshGroupPaintToolActionCommands);
	}
	else if (ExactCast<UMeshVertexPaintTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FMeshVertexPaintToolActionCommands);
	}
	else if (ExactCast<UMeshAttributePaintTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FMeshAttributePaintToolActionCommands);
	}
	else if (ExactCast<UDrawPolygonTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FDrawPolygonToolActionCommands);
	}
	else if (ExactCast<UMeshSelectionTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FMeshSelectionToolActionCommands);
	}
	else if (ExactCast<UEditMeshMaterialsTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FEditMeshMaterialsToolActionCommands);
	}
	else if (ExactCast<UPlaneCutTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FMeshPlaneCutToolActionCommands);
	}
	else if (ExactCast<UCubeGridTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FCubeGridToolActionCommands);
	}
	else if (ExactCast<UEditMeshPolygonsTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FEditMeshPolygonsToolActionCommands);
	}
	else if (ExactCast<UDrawAndRevolveTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FDrawAndRevolveToolActionCommands);
	}
	else
	{
		UPDATE_BINDING(FModelingToolActionCommands);
	}
}





#define DEFINE_TOOL_ACTION_COMMANDS(CommandsClassName, ContextNameString, SettingsDialogString, ToolClassName ) \
CommandsClassName::CommandsClassName() : TInteractiveToolCommands<CommandsClassName>( \
ContextNameString, NSLOCTEXT("Contexts", ContextNameString, SettingsDialogString), NAME_None, FAppStyle::GetAppStyleSetName()) {} \
void CommandsClassName::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) \
{\
	ToolCDOs.Add(GetMutableDefault<ToolClassName>()); \
}




DEFINE_TOOL_ACTION_COMMANDS(FSculptToolActionCommands, "ModelingToolsSculptTool", "Modeling Tools - Sculpt Tool", UDynamicMeshSculptTool);
DEFINE_TOOL_ACTION_COMMANDS(FVertexSculptToolActionCommands, "ModelingToolsVertexSculptTool", "Modeling Tools - Vertex Sculpt Tool", UMeshVertexSculptTool);
DEFINE_TOOL_ACTION_COMMANDS(FMeshGroupPaintToolActionCommands, "ModelingToolsMeshGroupPaintTool", "Modeling Tools - Group Paint Tool", UMeshGroupPaintTool);
DEFINE_TOOL_ACTION_COMMANDS(FMeshVertexPaintToolActionCommands, "ModelingToolsMeshVertexPaintTool", "Modeling Tools - Vertex Paint Tool", UMeshVertexPaintTool);
DEFINE_TOOL_ACTION_COMMANDS(FMeshAttributePaintToolActionCommands, "ModelingToolsMeshAttributePaintTool", "Modeling Tools - Attribute Paint Tool", UMeshAttributePaintTool);
DEFINE_TOOL_ACTION_COMMANDS(FTransformToolActionCommands, "ModelingToolsTransformTool", "Modeling Tools - Transform Tool", UTransformMeshesTool);
DEFINE_TOOL_ACTION_COMMANDS(FDrawPolygonToolActionCommands, "ModelingToolsDrawPolygonTool", "Modeling Tools - Draw Polygon Tool", UDrawPolygonTool);
DEFINE_TOOL_ACTION_COMMANDS(FMeshSelectionToolActionCommands, "ModelingToolsMeshSelectionTool", "Modeling Tools - Mesh Selection Tool", UMeshSelectionTool);
DEFINE_TOOL_ACTION_COMMANDS(FEditMeshMaterialsToolActionCommands, "ModelingToolsEditMeshMaterials", "Modeling Tools - Edit Materials Tool", UEditMeshMaterialsTool);
DEFINE_TOOL_ACTION_COMMANDS(FMeshPlaneCutToolActionCommands, "ModelingToolsMeshPlaneCutTool", "Modeling Tools - Mesh Plane Cut Tool", UPlaneCutTool);
DEFINE_TOOL_ACTION_COMMANDS(FCubeGridToolActionCommands, "ModelingToolsCubeGridTool", "Modeling Tools - Cube Grid Tool", UCubeGridTool);
DEFINE_TOOL_ACTION_COMMANDS(FEditMeshPolygonsToolActionCommands, "ModelingToolsEditMeshPolygonsTool", "Modeling Tools - Edit Mesh Polygons Tool", UEditMeshPolygonsTool);
DEFINE_TOOL_ACTION_COMMANDS(FDrawAndRevolveToolActionCommands, "ModelingToolsDrawAndRevolveTool", "Modeling Tools - Draw-and-Revolve Tool", UDrawAndRevolveTool);





#undef LOCTEXT_NAMESPACE

