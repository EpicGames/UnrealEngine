// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingToolsEditablePaletteConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelingToolsEditablePaletteConfig)

TObjectPtr<UModelingModeEditableToolPaletteConfig> UModelingModeEditableToolPaletteConfig::Instance = nullptr;

void UModelingModeEditableToolPaletteConfig::Initialize()
{
	if(!Instance)
	{
		Instance = NewObject<UModelingModeEditableToolPaletteConfig>(); 
		Instance->AddToRoot();
	}
}

FEditableToolPaletteSettings* UModelingModeEditableToolPaletteConfig::GetMutablePaletteConfig(const FName& InstanceName)
{
	if (InstanceName.IsNone())
	{
		return nullptr;
	}

	return &EditableToolPalettes.FindOrAdd(InstanceName);
}

const FEditableToolPaletteSettings* UModelingModeEditableToolPaletteConfig::GetConstPaletteConfig(const FName& InstanceName)
{
	if (InstanceName.IsNone())
	{
		return nullptr;
	}

	return EditableToolPalettes.Find(InstanceName);
}

void UModelingModeEditableToolPaletteConfig::SavePaletteConfig(const FName&)
{
	SaveEditorConfig();
}
