// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "EditorConfigBase.h"
#include "ToolkitBuilderConfig.h"
#include "UObject/ObjectPtr.h"

#include "ModelingToolsEditablePaletteConfig.generated.h"

/* Implementation of IEditableToolPaletteConfigManager specific to ModelingMode, currently needed because we cannot have
 * one at the FModeToolkit level due to EditorConfig depending on UnrealEd which is where the mode toolkit lives
 */
UCLASS(EditorConfig="EditableToolPalette")
class UModelingModeEditableToolPaletteConfig : public UEditorConfigBase, public IEditableToolPaletteConfigManager
{
	GENERATED_BODY()
	
public:

	// IEditableToolPaletteConfigManager Interface
	virtual FEditableToolPaletteSettings* GetMutablePaletteConfig(const FName& InstanceName) override;
	virtual const FEditableToolPaletteSettings* GetConstPaletteConfig(const FName& InstanceName) override;
	virtual void SavePaletteConfig(const FName& InstanceName) override;

	static void Initialize();
	static UModelingModeEditableToolPaletteConfig* Get() { return Instance; }
	static IEditableToolPaletteConfigManager* GetAsConfigManager() { return Instance; }

	UPROPERTY(meta=(EditorConfig))
	TMap<FName, FEditableToolPaletteSettings> EditableToolPalettes;

private:

	static TObjectPtr<UModelingModeEditableToolPaletteConfig> Instance;
};