// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingToolsEditorModeModule.h"

#include "ModelingToolsActions.h"
#include "ModelingToolsManagerActions.h"
#include "ModelingToolsEditorModeStyle.h"


#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "DetailsCustomizations/ModelingToolPropertyCustomizations.h"
#include "DetailsCustomizations/ModelingToolsBrushSizeCustomization.h"
#include "DetailsCustomizations/MeshVertexSculptToolCustomizations.h"
#include "DetailsCustomizations/MeshVertexPaintToolCustomizations.h"
#include "DetailsCustomizations/BakeMeshAttributeToolCustomizations.h"
#include "DetailsCustomizations/BakeTransformToolCustomizations.h"
#include "DetailsCustomizations/MeshTopologySelectionMechanicCustomization.h"

#include "PropertySets/AxisFilterPropertyType.h"
#include "PropertySets/ColorChannelFilterPropertyType.h"
#include "Selection/MeshTopologySelectionMechanic.h"
#include "MeshVertexSculptTool.h"
#include "MeshVertexPaintTool.h"
#include "BakeMeshAttributeMapsTool.h"
#include "BakeMultiMeshAttributeMapsTool.h"
#include "BakeMeshAttributeVertexTool.h"
#include "BakeTransformTool.h"
#include "MeshVertexAttributePaintToolBase.h"
#include "DetailsCustomizations/MeshVertexAttributePaintToolDetailCustomization.h"
#include "Sculpting/KelvinletBrushOp.h"
#include "Sculpting/MeshInflateBrushOps.h"
#include "Sculpting/MeshMoveBrushOps.h"
#include "Sculpting/MeshPinchBrushOps.h"
#include "Sculpting/MeshPlaneBrushOps.h"
#include "Sculpting/MeshSculptBrushOps.h"
#include "Sculpting/MeshSmoothingBrushOps.h"
#include "Properties/MeshSculptLayerProperties.h"


#define LOCTEXT_NAMESPACE "FModelingToolsEditorModeModule"

void FModelingToolsEditorModeModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FModelingToolsEditorModeModule::OnPostEngineInit);
}

void FModelingToolsEditorModeModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	FModelingToolActionCommands::UnregisterAllToolActions();
	FModelingToolsManagerCommands::Unregister();
	FModelingModeActionCommands::Unregister();

	// Unregister customizations
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		for (FName ClassName : ClassesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(ClassName);
		}
		for (FName PropertyName : PropertiesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(PropertyName);
		}
	}

	// Unregister slate style overrides
	FModelingToolsEditorModeStyle::Shutdown();
}


void FModelingToolsEditorModeModule::OnPostEngineInit()
{
	// Register slate style overrides
	FModelingToolsEditorModeStyle::Initialize();

	FModelingToolActionCommands::RegisterAllToolActions();
	FModelingToolsManagerCommands::Register();
	FModelingModeActionCommands::Register();

	// same as ClassesToUnregisterOnShutdown but for properties, there is none right now
	PropertiesToUnregisterOnShutdown.Reset();
	ClassesToUnregisterOnShutdown.Reset();


	// Register details view customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	/// Sculpt
	PropertyModule.RegisterCustomPropertyTypeLayout("ModelingToolsAxisFilter", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FModelingToolsAxisFilterCustomization::MakeInstance));
	PropertiesToUnregisterOnShutdown.Add(FModelingToolsAxisFilter::StaticStruct()->GetFName());
	PropertyModule.RegisterCustomPropertyTypeLayout("ModelingToolsColorChannelFilter", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FModelingToolsColorChannelFilterCustomization::MakeInstance));
	PropertiesToUnregisterOnShutdown.Add(FModelingToolsColorChannelFilter::StaticStruct()->GetFName());
	PropertyModule.RegisterCustomPropertyTypeLayout("BrushToolRadius", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FModelingToolsBrushSizeCustomization::MakeInstance));
	PropertiesToUnregisterOnShutdown.Add(FBrushToolRadius::StaticStruct()->GetFName());
	PropertyModule.RegisterCustomClassLayout("SculptBrushProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FSculptBrushPropertiesDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(USculptBrushProperties::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("VertexBrushSculptProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FVertexBrushSculptPropertiesDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UVertexBrushSculptProperties::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("VertexBrushAlphaProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FVertexBrushAlphaPropertiesDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UVertexBrushAlphaProperties::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("MeshSculptLayerProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptLayerPropertiesDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UMeshSculptLayerProperties::StaticClass()->GetFName());
	// Sculpt - BrushOpProps
	PropertyModule.RegisterCustomClassLayout("PinchBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UPinchBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UPinchBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("InflateBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UInflateBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UInflateBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("SmoothBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<USmoothBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(USmoothBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("SmoothFillBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<USmoothFillBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(USmoothFillBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("FlattenBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UFlattenBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UFlattenBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("EraseBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UEraseBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UEraseBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("StandardSculptBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UStandardSculptBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UStandardSculptBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("ViewAlignedSculptBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UViewAlignedSculptBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UViewAlignedSculptBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("SculptMaxBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<USculptMaxBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(USculptMaxBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("FixedPlaneBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UFixedPlaneBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UFixedPlaneBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("ViewAlignedPlaneBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UViewAlignedPlaneBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UViewAlignedPlaneBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("PlaneBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UPlaneBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UPlaneBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("MoveBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UMoveBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UMoveBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("SecondarySmoothBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<USecondarySmoothBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(USecondarySmoothBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("ScaleKelvinletBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UScaleKelvinletBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UScaleKelvinletBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("PullKelvinletBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UPullKelvinletBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UPullKelvinletBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("SharpPullKelvinletBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<USharpPullKelvinletBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(USharpPullKelvinletBrushOpProps::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("TwistKelvinletBrushOpProps", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSculptBrushOpPropertiesDetails<UTwistKelvinletBrushOpProps>::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UTwistKelvinletBrushOpProps::StaticClass()->GetFName());

	// Paint
	PropertyModule.RegisterCustomClassLayout("VertexPaintBasicProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FVertexPaintBasicPropertiesDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UVertexPaintBasicProperties::StaticClass()->GetFName());

	/// Bake
	PropertyModule.RegisterCustomClassLayout("BakeMeshAttributeMapsToolProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FBakeMeshAttributeMapsToolDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UBakeMeshAttributeMapsToolProperties::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("BakeMultiMeshAttributeMapsToolProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FBakeMultiMeshAttributeMapsToolDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UBakeMultiMeshAttributeMapsToolProperties::StaticClass()->GetFName());
	PropertyModule.RegisterCustomClassLayout("BakeMeshAttributeVertexToolProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FBakeMeshAttributeVertexToolDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UBakeMeshAttributeVertexToolProperties::StaticClass()->GetFName());
	// PolyEd
	PropertyModule.RegisterCustomClassLayout("MeshTopologySelectionMechanicProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshTopologySelectionMechanicPropertiesDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UMeshTopologySelectionMechanicProperties::StaticClass()->GetFName());

	PropertyModule.RegisterCustomClassLayout("BakeTransformToolProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FBakeTransformToolDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UBakeTransformToolProperties::StaticClass()->GetFName());

	// Vertex Attribute Paint
	PropertyModule.RegisterCustomClassLayout(UMeshVertexAttributePaintToolProperties::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&UE::MeshVertexAttributePaintToolBase::FVertexAttributePaintToolDetailCustomization::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UMeshVertexAttributePaintToolProperties::StaticClass()->GetFName());
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FModelingToolsEditorModeModule, ModelingToolsEditorMode)
