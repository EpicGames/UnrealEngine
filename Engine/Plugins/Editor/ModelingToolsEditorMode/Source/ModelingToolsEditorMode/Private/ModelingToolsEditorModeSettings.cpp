// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingToolsEditorModeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelingToolsEditorModeSettings)


#define LOCTEXT_NAMESPACE "ModelingToolsEditorModeSettings"


FText UModelingToolsEditorModeSettings::GetSectionText() const 
{ 
	return LOCTEXT("ModelingModeSettingsName", "Modeling Mode"); 
}

FText UModelingToolsEditorModeSettings::GetSectionDescription() const
{
	return LOCTEXT("ModelingModeSettingsDescription", "Configure the Modeling Tools Editor Mode plugin");
}


FText UModelingToolsModeCustomizationSettings::GetSectionText() const 
{ 
	return LOCTEXT("ModelingModeSettingsName", "Modeling Mode"); 
}

FText UModelingToolsModeCustomizationSettings::GetSectionDescription() const
{
	return LOCTEXT("ModelingModeSettingsDescription", "Configure the Modeling Tools Editor Mode plugin");
}


#undef LOCTEXT_NAMESPACE
