// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingToolsEditorModeStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Misc/Paths.h"
#include "Styling/CoreStyle.h"
#include "Interfaces/IPluginManager.h"
#include "SlateOptMacros.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/ToolBarStyle.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FModelingToolsEditorModeStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define RootToContentDir StyleSet->RootToContentDir

FString FModelingToolsEditorModeStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("ModelingToolsEditorMode"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr< FSlateStyleSet > FModelingToolsEditorModeStyle::StyleSet = nullptr;
TSharedPtr< class ISlateStyle > FModelingToolsEditorModeStyle::Get() { return StyleSet; }

FName FModelingToolsEditorModeStyle::GetStyleSetName()
{
	static FName ModelingToolsStyleName(TEXT("ModelingToolsStyle"));
	return ModelingToolsStyleName;
}

const FSlateBrush* FModelingToolsEditorModeStyle::GetBrush(FName PropertyName, const ANSICHAR* Specifier)
{
	return Get()->GetBrush(PropertyName, Specifier);
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FModelingToolsEditorModeStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon28x28(28.0f, 28.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon120(120.0f, 120.0f);

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));

	// If we get asked for something that we don't set, we should default to editor style
	StyleSet->SetParentStyleName("EditorStyle");

	StyleSet->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Editor/ModelingToolsEditorMode/Content"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	// Shared editors
	//{
	//	StyleSet->Set("Paper2D.Common.ViewportZoomTextStyle", FTextBlockStyle(NormalText)
	//		.SetFont(DEFAULT_FONT("BoldCondensed", 16))
	//	);

	//	StyleSet->Set("Paper2D.Common.ViewportTitleTextStyle", FTextBlockStyle(NormalText)
	//		.SetFont(DEFAULT_FONT("Regular", 18))
	//		.SetColorAndOpacity(FLinearColor(1.0, 1.0f, 1.0f, 0.5f))
	//	);

	//	StyleSet->Set("Paper2D.Common.ViewportTitleBackground", new BOX_BRUSH("Old/Graph/GraphTitleBackground", FMargin(0)));
	//}

	// Tool Manager icons
	{
		// Accept/Cancel/Complete active tool

		StyleSet->Set("LevelEditor.ModelingToolsMode", new IMAGE_BRUSH_SVG("Starship/geometry", FVector2D(20.0f, 20.0f)));

		StyleSet->Set("ModelingMode.DefaultSettings", new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/GeneralTools/Settings_40x.png")), Icon20x20));
		StyleSet->Set("ModelingMode.SubToolArrow", new FSlateImageBrush(StyleSet->RootToCoreContentDir(
			// This arrow is 25x13, but 20x13 looks better
			TEXT("../Editor/Slate/Persona/BlendSpace/arrow_right_12x.png")), FVector2D(20,13)));

		// NOTE:  Old-style, need to be replaced: 
		StyleSet->Set("ModelingToolsManagerCommands.CancelActiveTool", new IMAGE_PLUGIN_BRUSH("Icons/icon_ActiveTool_Cancel_40x", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.CancelActiveTool.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_ActiveTool_Cancel_40x", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.AcceptActiveTool", new IMAGE_PLUGIN_BRUSH("Icons/icon_ActiveTool_Accept_40x", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.AcceptActiveTool.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_ActiveTool_Accept_40x", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.CompleteActiveTool", new IMAGE_PLUGIN_BRUSH("Icons/icon_ActiveTool_Accept_40x", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.CompleteActiveTool.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_ActiveTool_Accept_40x", Icon20x20));

		StyleSet->Set("ModelingToolsManagerCommands.LoadFavoritesTools", new IMAGE_BRUSH_SVG( "Icons/LoadFavoritesTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadSelectionTools", new IMAGE_BRUSH_SVG("Icons/ModSelectionObject_16", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadShapesTools", new IMAGE_BRUSH_SVG( "Icons/LoadShapesTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadCreateTools", new IMAGE_BRUSH_SVG( "Icons/LoadCreateTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadPolyTools", new IMAGE_BRUSH_SVG( "Icons/LoadPolyTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadTriTools", new IMAGE_BRUSH_SVG( "Icons/LoadTriTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadDeformTools", new IMAGE_BRUSH_SVG( "Icons/LoadDeformTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadTransformTools", new IMAGE_BRUSH_SVG( "Icons/LoadTransformTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadMeshOpsTools", new IMAGE_BRUSH_SVG( "Icons/LoadMeshOpsTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadVoxOpsTools", new IMAGE_BRUSH_SVG( "Icons/LoadVoxOpsTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadAttributesTools", new IMAGE_BRUSH_SVG( "Icons/LoadAttributesTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadUVsTools", new IMAGE_BRUSH_SVG( "Icons/LoadUVsTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadBakingTools", new IMAGE_BRUSH_SVG( "Icons/LoadBakingTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadVolumeTools", new IMAGE_BRUSH_SVG( "Icons/LoadVolumeTools", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadLodsTools", new IMAGE_BRUSH_SVG( "Icons/LoadLodsTools", Icon20x20 ) );
		
		StyleSet->Set("ModelingToolsManagerCommands.LoadSkinTools", new IMAGE_BRUSH_SVG( "Icons/Skin", Icon20x20 ) );
		StyleSet->Set("ModelingToolsManagerCommands.LoadSkeletonTools", new IMAGE_BRUSH_SVG( "Icons/SkeletalEditor_20", Icon20x20 ) );

		StyleSet->Set("ModelingToolsManagerCommands.BeginShapeSprayTool", 				new IMAGE_PLUGIN_BRUSH("Icons/ShapeSpray_40x",	Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginShapeSprayTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ShapeSpray_40x",	Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginMeshSpaceDeformerTool", 		new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_Displace_40x",		Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginMeshSpaceDeformerTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_Displace_40x",		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolygonOnMeshTool", 			new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_PolygonOnMesh_40x",	Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolygonOnMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_PolygonOnMesh_40x",	Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginParameterizeMeshTool", 		new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_UVGenerate_40x",	Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginParameterizeMeshTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_UVGenerate_40x",	Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyGroupsTool", 				new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_PolyGroups_40x",	Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyGroupsTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_PolyGroups_40x",	Icon20x20));


		// Modes Palette Toolbar Icons
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddBoxPrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/ModelingBox", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddBoxPrimitiveTool.Small", 		new IMAGE_BRUSH_SVG("Icons/ModelingBox",		Icon40x40));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddCapsulePrimitiveTool",			new IMAGE_BRUSH_SVG("Icons/ModelingCapsule", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddCapsulePrimitiveTool.Small",	new IMAGE_BRUSH_SVG("Icons/ModelingCapsule", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddCylinderPrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/ModelingCylinder", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddCylinderPrimitiveTool.Small", 		new IMAGE_BRUSH_SVG("Icons/ModelingCylinder",		Icon40x40));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddConePrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/ModelingCone", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddConePrimitiveTool.Small", 		new IMAGE_BRUSH_SVG("Icons/ModelingCone",		Icon40x40));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddArrowPrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/ModelingArrow", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddArrowPrimitiveTool.Small", 		new IMAGE_BRUSH_SVG("Icons/ModelingArrow",		Icon40x40));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddRectanglePrimitiveTool", 			new IMAGE_PLUGIN_BRUSH("Icons/ModelingRectangle_x20", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddRectanglePrimitiveTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingRectangle_x40",		Icon40x40));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddDiscPrimitiveTool", 			new IMAGE_PLUGIN_BRUSH("Icons/ModelingDisc_x20", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddDiscPrimitiveTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingDisc_x40",		Icon40x40));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddTorusPrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/ModelingTorus", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddTorusPrimitiveTool.Small", 		new IMAGE_BRUSH_SVG("Icons/ModelingTorus",		Icon40x40));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddSpherePrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/ModelingSphere", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddSpherePrimitiveTool.Small", 		new IMAGE_BRUSH_SVG("Icons/ModelingSphere",		Icon40x40));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddStairsPrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/Staircase", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddStairsPrimitiveTool.Small",		new IMAGE_BRUSH_SVG("Icons/Staircase", Icon40x40));
		StyleSet->Set("ModelingToolsManagerCommands.BeginDrawSplineTool",	     	        new IMAGE_BRUSH_SVG("Icons/GeometryDrawSpline", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginDrawSplineTool.Small",		        new IMAGE_BRUSH_SVG("Icons/GeometryDrawSpline", Icon40x40));

		StyleSet->Set("ModelingToolsManagerCommands.BeginDrawPolygonTool", 				new IMAGE_PLUGIN_BRUSH("Icons/DrawPolygon_40x",		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginDrawPolygonTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/DrawPolygon_40x", 	Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddPatchTool",					new IMAGE_PLUGIN_BRUSH("Icons/Patch_40x",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddPatchTool.Small",			new IMAGE_PLUGIN_BRUSH("Icons/Patch_40x",			Icon20x20));

		StyleSet->Set("ModelingToolsManagerCommands.BeginSmoothMeshTool", 				new IMAGE_PLUGIN_BRUSH("Icons/ModelingSmooth_x40", 			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSmoothMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingSmooth_x40", 			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSculptMeshTool", 				new IMAGE_PLUGIN_BRUSH("Icons/Sculpt_40x", 			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSculptMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Sculpt_40x", 			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyEditTool", 				new IMAGE_PLUGIN_BRUSH("Icons/PolyEdit_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyEditTool.Small", 			new IMAGE_PLUGIN_BRUSH("Icons/PolyEdit_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSubdividePolyTool",			new IMAGE_BRUSH_SVG("Icons/ModelingSubD",		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSubdividePolyTool.Small",		new IMAGE_BRUSH_SVG("Icons/ModelingSubD",		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginTriEditTool", 				new IMAGE_PLUGIN_BRUSH("Icons/TriEdit_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginTriEditTool.Small", 			new IMAGE_PLUGIN_BRUSH("Icons/TriEdit_40x", 		Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginPolyDeformTool", 			new IMAGE_PLUGIN_BRUSH("Icons/PolyEdit_40x", 		Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginPolyDeformTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/PolyEdit_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginDisplaceMeshTool", 			new IMAGE_PLUGIN_BRUSH("Icons/Displace_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginDisplaceMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Displace_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginTransformMeshesTool", 			new IMAGE_PLUGIN_BRUSH("Icons/Transform_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginTransformMeshesTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/Transform_40x", 		Icon20x20));

		StyleSet->Set("ModelingToolsManagerCommands.BeginRemeshSculptMeshTool", 		new IMAGE_PLUGIN_BRUSH("Icons/DynaSculpt_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginRemeshSculptMeshTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/DynaSculpt_40x",		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginRemeshMeshTool", 				new IMAGE_PLUGIN_BRUSH("Icons/Remesh_40x", 			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginRemeshMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Remesh_40x", 			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginProjectToTargetTool", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingRemeshToTarget_x40",	Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginProjectToTargetTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/ModelingRemeshToTarget_x40",	Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginProjectToTargetTool",			new IMAGE_PLUGIN_BRUSH("",			Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginProjectToTargetTool.Small",	new IMAGE_PLUGIN_BRUSH("",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSimplifyMeshTool", 			new IMAGE_PLUGIN_BRUSH("Icons/Simplify_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSimplifyMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Simplify_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginEditNormalsTool", 			new IMAGE_PLUGIN_BRUSH("Icons/Normals_40x",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginEditNormalsTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Normals_40x",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginEditTangentsTool", 			new IMAGE_PLUGIN_BRUSH("Icons/ModelingTangents_x40",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginEditTangentsTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingTangents_x40",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginUVSeamEditTool", 			new IMAGE_PLUGIN_BRUSH("Icons/ModelingUVSeamEdit_x40",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginUVSeamEditTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingUVSeamEdit_x40",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginBakeMeshAttributeMapsTool", 			new IMAGE_BRUSH_SVG("Icons/BakeTexture",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginBakeMeshAttributeMapsTool.Small", 		new IMAGE_BRUSH_SVG("Icons/BakeTexture",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginBakeMultiMeshAttributeMapsTool", 			new IMAGE_BRUSH_SVG("Icons/BakeAll",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginBakeMultiMeshAttributeMapsTool.Small", 		new IMAGE_BRUSH_SVG("Icons/BakeAll",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginBakeRenderCaptureTool", 			new IMAGE_BRUSH_SVG("Icons/BakeRenderCapture",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginBakeRenderCaptureTool.Small", 		new IMAGE_BRUSH_SVG("Icons/BakeRenderCapture",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginBakeMeshAttributeVertexTool", new IMAGE_BRUSH_SVG("Icons/BakeVertex", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginBakeMeshAttributeVertexTool.Small", new IMAGE_BRUSH_SVG("Icons/BakeVertex", Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginRemoveOccludedTrianglesTool", 				new IMAGE_PLUGIN_BRUSH("Icons/Jacket_40x",			Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginRemoveOccludedTrianglesTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Jacket_40x",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginHoleFillTool", 				new IMAGE_PLUGIN_BRUSH("Icons/ModelingHoleFill_x40",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginHoleFillTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingHoleFill_x40",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginUVProjectionTool", 			new IMAGE_PLUGIN_BRUSH("Icons/UVProjection_40x", 	Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginUVProjectionTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/UVProjection_40x", 	Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginUVLayoutTool", 			new IMAGE_PLUGIN_BRUSH("Icons/UVLayout_40x", 	Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginUVLayoutTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/UVLayout_40x", 	Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginVoxelMergeTool", 				new IMAGE_PLUGIN_BRUSH("Icons/VoxMerge_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginVoxelMergeTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/VoxMerge_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginVoxelBooleanTool", 			new IMAGE_PLUGIN_BRUSH("Icons/VoxBoolean_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginVoxelBooleanTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/VoxBoolean_40x", 		Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginSelfUnionTool",				new IMAGE_PLUGIN_BRUSH("Icons/MeshMerge_40x",		Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginSelfUnionTool.Small",			new IMAGE_PLUGIN_BRUSH("Icons/MeshMerge_40x",		Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginMeshBooleanTool",				new IMAGE_PLUGIN_BRUSH("Icons/Boolean_40x",			Icon20x20));
		//StyleSet->Set("ModelingToolsManagerCommands.BeginMeshBooleanTool.Small",		new IMAGE_PLUGIN_BRUSH("Icons/Boolean_40x",			Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPlaneCutTool", 				new IMAGE_PLUGIN_BRUSH("Icons/PlaneCut_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPlaneCutTool.Small", 			new IMAGE_PLUGIN_BRUSH("Icons/PlaneCut_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMirrorTool", 				    new IMAGE_PLUGIN_BRUSH("Icons/ModelingMirror_x40", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMirrorTool.Small", 			new IMAGE_PLUGIN_BRUSH("Icons/ModelingMirror_x40", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginOffsetMeshTool", 				new IMAGE_PLUGIN_BRUSH("Icons/ModelingOffset_x40", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginOffsetMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingOffset_x40", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginDisplaceMeshTool", 			new IMAGE_PLUGIN_BRUSH("Icons/ModelingDisplace_x40", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginDisplaceMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingDisplace_x40", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshSelectionTool", 			new IMAGE_PLUGIN_BRUSH("Icons/MeshSelect_40x",		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshSelectionTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/MeshSelect_40x",		Icon20x20));

		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshInspectorTool", 			new IMAGE_PLUGIN_BRUSH("Icons/Inspector_40x",		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshInspectorTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Inspector_40x",		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginWeldEdgesTool", 				new IMAGE_PLUGIN_BRUSH("Icons/WeldEdges_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginWeldEdgesTool.Small", 			new IMAGE_PLUGIN_BRUSH("Icons/WeldEdges_40x", 		Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAttributeEditorTool", 			new IMAGE_PLUGIN_BRUSH("Icons/AttributeEditor_40x", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAttributeEditorTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/AttributeEditor_40x", Icon20x20));

		StyleSet->Set("ModelingToolsManagerCommands.BeginAlignObjectsTool",                  new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/GeneralTools/Align_40x.png")), Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAlignObjectsTool.Small",            new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/GeneralTools/Align_40x.png")), Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginTransferMeshTool",                  new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/GeneralTools/Next_40x.png")), Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginTransferMeshTool.Small",            new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/GeneralTools/Next_40x.png")), Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginGlobalUVGenerateTool",              new IMAGE_PLUGIN_BRUSH("Icons/AutoUnwrap_40x",       Icon20x20));      
		StyleSet->Set("ModelingToolsManagerCommands.BeginGlobalUVGenerateTool.Small",        new IMAGE_PLUGIN_BRUSH("Icons/AutoUnwrap_40x",       Icon20x20));      
		StyleSet->Set("ModelingToolsManagerCommands.BeginBakeTransformTool",                 new IMAGE_BRUSH_SVG("Icons/GeometryBakeXForm",        Icon20x20));      
		StyleSet->Set("ModelingToolsManagerCommands.BeginBakeTransformTool.Small",           new IMAGE_BRUSH_SVG("Icons/GeometryBakeXForm",        Icon20x20));      
		StyleSet->Set("ModelingToolsManagerCommands.BeginCombineMeshesTool",                 new IMAGE_BRUSH_SVG("Icons/GeometryCombine",          Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginCombineMeshesTool.Small",           new IMAGE_BRUSH_SVG("Icons/GeometryCombine",          Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginDuplicateMeshesTool",               new IMAGE_PLUGIN_BRUSH("Icons/Duplicate_40x",        Icon20x20));   
		StyleSet->Set("ModelingToolsManagerCommands.BeginDuplicateMeshesTool.Small",         new IMAGE_PLUGIN_BRUSH("Icons/Duplicate_40x",        Icon20x20));   
		StyleSet->Set("ModelingToolsManagerCommands.BeginEditMeshMaterialsTool",             new IMAGE_PLUGIN_BRUSH("Icons/EditMats_40x",         Icon20x20));     
		StyleSet->Set("ModelingToolsManagerCommands.BeginEditMeshMaterialsTool.Small",       new IMAGE_PLUGIN_BRUSH("Icons/EditMats_40x",         Icon20x20));     
		StyleSet->Set("ModelingToolsManagerCommands.BeginEditPivotTool",                     new IMAGE_PLUGIN_BRUSH("Icons/EditPivot_40x",        Icon20x20));      
		StyleSet->Set("ModelingToolsManagerCommands.BeginEditPivotTool.Small",               new IMAGE_PLUGIN_BRUSH("Icons/EditPivot_40x",        Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddPivotActorTool",                 new IMAGE_BRUSH_SVG("Icons/Pivot",                   Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginAddPivotActorTool.Small",           new IMAGE_BRUSH_SVG("Icons/Pivot",                   Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginGroupUVGenerateTool",               new IMAGE_PLUGIN_BRUSH("Icons/GroupUnwrap_40x",      Icon20x20));       
		StyleSet->Set("ModelingToolsManagerCommands.BeginGroupUVGenerateTool.Small",         new IMAGE_PLUGIN_BRUSH("Icons/GroupUnwrap_40x",      Icon20x20));       
		StyleSet->Set("ModelingToolsManagerCommands.BeginRemoveOccludedTrianglesTool",       new IMAGE_PLUGIN_BRUSH("Icons/Jacketing_40x",        Icon20x20));     
		StyleSet->Set("ModelingToolsManagerCommands.BeginRemoveOccludedTrianglesTool.Small", new IMAGE_PLUGIN_BRUSH("Icons/Jacketing_40x",        Icon20x20));     
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolygonCutTool",                    new IMAGE_PLUGIN_BRUSH("Icons/PolyCut_40x",          Icon20x20));   
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolygonCutTool.Small",              new IMAGE_PLUGIN_BRUSH("Icons/PolyCut_40x",          Icon20x20));   
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyDeformTool",                    new IMAGE_PLUGIN_BRUSH("Icons/PolyDeform_40x",       Icon20x20));      
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyDeformTool.Small",              new IMAGE_PLUGIN_BRUSH("Icons/PolyDeform_40x",       Icon20x20));      
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyGroupsTool",                    new IMAGE_PLUGIN_BRUSH("Icons/PolyGroups_40x",       Icon20x20));      
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyGroupsTool.Small",              new IMAGE_PLUGIN_BRUSH("Icons/PolyGroups_40x",       Icon20x20));      
		StyleSet->Set("ModelingToolsManagerCommands.BeginDrawPolyPathTool",                  new IMAGE_PLUGIN_BRUSH("Icons/PolyPath_40x",         Icon20x20));    
		StyleSet->Set("ModelingToolsManagerCommands.BeginDrawPolyPathTool.Small",            new IMAGE_PLUGIN_BRUSH("Icons/PolyPath_40x",         Icon20x20));    
		StyleSet->Set("ModelingToolsManagerCommands.BeginDrawAndRevolveTool",                new IMAGE_BRUSH_SVG("Icons/ModelingDrawAndRevolve",  Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginDrawAndRevolveTool.Small",          new IMAGE_BRUSH_SVG("Icons/ModelingDrawAndRevolve",  Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginRevolveBoundaryTool",               new IMAGE_BRUSH_SVG("Icons/ModelingRevolveBoundary", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginRevolveBoundaryTool.Small",         new IMAGE_BRUSH_SVG("Icons/ModelingRevolveBoundary", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginRevolveSplineTool",                 new IMAGE_BRUSH_SVG("Icons/ModelingRevolveSpline",   Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginRevolveSplineTool.Small",           new IMAGE_BRUSH_SVG("Icons/ModelingRevolveSpline",   Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginTriangulateSplinesTool",            new IMAGE_BRUSH_SVG("Icons/ModelingTriangulateSpline",   Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginTriangulateSplinesTool.Small",      new IMAGE_BRUSH_SVG("Icons/ModelingTriangulateSpline",   Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginISMEditorTool",						new IMAGE_BRUSH_SVG("Icons/ModelingISMEditor",       Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginISMEditorTool.Small",				new IMAGE_BRUSH_SVG("Icons/ModelingISMEditor",       Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginCubeGridTool",                      new IMAGE_BRUSH_SVG("Icons/CubeGrid",                Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginCubeGridTool.Small",                new IMAGE_BRUSH_SVG("Icons/CubeGrid",                Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshBooleanTool",                   new IMAGE_PLUGIN_BRUSH("Icons/ModelingMeshBoolean_x40", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshBooleanTool.Small",             new IMAGE_PLUGIN_BRUSH("Icons/ModelingMeshBoolean_x20", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshTrimTool",				 new IMAGE_BRUSH_SVG("Icons/ModelingTrim", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshTrimTool.Small",			 new IMAGE_BRUSH_SVG("Icons/ModelingTrim", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginCutMeshWithMeshTool",               new IMAGE_BRUSH_SVG("Icons/ModelingMeshCut", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginCutMeshWithMeshTool.Small",         new IMAGE_BRUSH_SVG("Icons/ModelingMeshCut", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelfUnionTool",                     new IMAGE_PLUGIN_BRUSH("Icons/ModelingSelfUnion_x40", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelfUnionTool.Small",               new IMAGE_PLUGIN_BRUSH("Icons/ModelingSelfUnion_x20", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginVoxelSolidifyTool",                 new IMAGE_PLUGIN_BRUSH("Icons/ModelingVoxSolidify_x40", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginVoxelSolidifyTool.Small",           new IMAGE_PLUGIN_BRUSH("Icons/ModelingVoxSolidify_x20", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginVoxelBlendTool",                    new IMAGE_PLUGIN_BRUSH("Icons/ModelingVoxBlend_x40", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginVoxelBlendTool.Small",              new IMAGE_PLUGIN_BRUSH("Icons/ModelingVoxBlend_x20", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginVoxelMorphologyTool",               new IMAGE_PLUGIN_BRUSH("Icons/ModelingVoxMorphology_x40", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginVoxelMorphologyTool.Small",         new IMAGE_PLUGIN_BRUSH("Icons/ModelingVoxMorphology_x20", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshSpaceDeformerTool",             new IMAGE_PLUGIN_BRUSH("Icons/SpaceDeform_40x",      Icon20x20));       
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshSpaceDeformerTool.Small",       new IMAGE_PLUGIN_BRUSH("Icons/SpaceDeform_40x",      Icon20x20));       
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshAttributePaintTool",             new IMAGE_PLUGIN_BRUSH("Icons/ModelingAttributePaint_x40",      Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshAttributePaintTool.Small",       new IMAGE_PLUGIN_BRUSH("Icons/ModelingAttributePaint_x40",      Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginTransformUVIslandsTool",            new IMAGE_PLUGIN_BRUSH("Icons/TransformUVs_40x",     Icon20x20));         
		StyleSet->Set("ModelingToolsManagerCommands.BeginTransformUVIslandsTool.Small",      new IMAGE_PLUGIN_BRUSH("Icons/TransformUVs_40x",     Icon20x20));         
		StyleSet->Set("ModelingToolsManagerCommands.BeginUVLayoutTool",                      new IMAGE_PLUGIN_BRUSH("Icons/UVLayout_40x",         Icon20x20));    
		StyleSet->Set("ModelingToolsManagerCommands.BeginUVLayoutTool.Small",                new IMAGE_PLUGIN_BRUSH("Icons/UVLayout_40x",         Icon20x20));  
		StyleSet->Set("ModelingToolsManagerCommands.BeginUVTransferTool",                    new IMAGE_BRUSH_SVG("Icons/UVTransfer", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginUVTransferTool.Small",              new IMAGE_BRUSH_SVG("Icons/UVTransfer", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.LaunchUVEditor",                         new IMAGE_BRUSH_SVG("Icons/UVEditor", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.LaunchUVEditor.Small",                   new IMAGE_BRUSH_SVG("Icons/UVEditor", Icon20x20));

		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshGroupPaintTool", new IMAGE_BRUSH_SVG("Icons/GroupPaint", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshGroupPaintTool.Small", new IMAGE_BRUSH_SVG("Icons/GroupPaint", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginLatticeDeformerTool", new IMAGE_BRUSH_SVG("Icons/LatticeDeformation", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginLatticeDeformerTool.Small", new IMAGE_BRUSH_SVG("Icons/LatticeDeformation", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginConvertMeshesTool", new IMAGE_BRUSH_SVG("Icons/Convert_20", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginConvertMeshesTool.Small", new IMAGE_BRUSH_SVG("Icons/Convert_20", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSplitMeshesTool", new IMAGE_BRUSH_SVG("Icons/GeometrySplit", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSplitMeshesTool.Small", new IMAGE_BRUSH_SVG("Icons/GeometrySplit", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPatternTool", new IMAGE_BRUSH_SVG("Icons/ModelingPattern", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPatternTool.Small", new IMAGE_BRUSH_SVG("Icons/ModelingPattern", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginHarvestInstancesTool", new IMAGE_BRUSH_SVG("Icons/HarvestInstances", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginHarvestInstancesTool.Small", new IMAGE_BRUSH_SVG("Icons/HarvestInstances", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshVertexPaintTool", new IMAGE_BRUSH_SVG("Icons/PaintVertexColors", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshVertexPaintTool.Small", new IMAGE_BRUSH_SVG("Icons/PaintVertexColors", Icon20x20));


		StyleSet->Set("ModelingToolsManagerCommands.BeginVolumeToMeshTool",                  new IMAGE_PLUGIN_BRUSH("Icons/ModelingVol2Mesh_x40",         Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginVolumeToMeshTool.Small",            new IMAGE_PLUGIN_BRUSH("Icons/ModelingVol2Mesh_x40",         Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshToVolumeTool",                  new IMAGE_PLUGIN_BRUSH("Icons/ModelingMesh2Vol_x40",         Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginMeshToVolumeTool.Small",            new IMAGE_PLUGIN_BRUSH("Icons/ModelingMesh2Vol_x40",         Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginBspConversionTool",                 new IMAGE_PLUGIN_BRUSH("Icons/ModelingBSPConversion_x40",         Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginBspConversionTool.Small",           new IMAGE_PLUGIN_BRUSH("Icons/ModelingBSPConversion_x40",         Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPhysicsInspectorTool",              new IMAGE_BRUSH_SVG("Icons/InspectCollision",                     Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPhysicsInspectorTool.Small",        new IMAGE_BRUSH_SVG("Icons/InspectCollision",                     Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSetCollisionGeometryTool",          new IMAGE_PLUGIN_BRUSH("Icons/ModelingMeshToCollision_x40",         Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSetCollisionGeometryTool.Small",    new IMAGE_PLUGIN_BRUSH("Icons/ModelingMeshToCollision_x40",         Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginExtractCollisionGeometryTool",      new IMAGE_PLUGIN_BRUSH("Icons/ModelingCollisionToMesh_x40",         Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginExtractCollisionGeometryTool.Small",new IMAGE_PLUGIN_BRUSH("Icons/ModelingCollisionToMesh_x40",         Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSimpleCollisionEditorTool",         new IMAGE_BRUSH_SVG("Icons/SimpleCollisionEditor",                  Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSimpleCollisionEditorTool.Small",   new IMAGE_BRUSH_SVG("Icons/SimpleCollisionEditor",                  Icon20x20));

		StyleSet->Set("ModelingToolsManagerCommands.BeginGenerateStaticMeshLODAssetTool", new IMAGE_BRUSH_SVG("Icons/AutoLOD", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginGenerateStaticMeshLODAssetTool.Small", new IMAGE_BRUSH_SVG("Icons/AutoLOD", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginLODManagerTool", new IMAGE_BRUSH_SVG("Icons/ModelingLODManager", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginLODManagerTool.Small", new IMAGE_BRUSH_SVG("Icons/ModelingLODManager", Icon20x20));

		StyleSet->Set("ModelingToolsManagerCommands.BeginGroomCardsEditorTool", new IMAGE_BRUSH_SVG("Icons/CardsEditor", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginGroomCardsEditorTool.Small", new IMAGE_BRUSH_SVG("Icons/CardsEditor", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginGenerateLODMeshesTool", new IMAGE_BRUSH_SVG("Icons/GenLODs", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginGenerateLODMeshesTool.Small", new IMAGE_BRUSH_SVG("Icons/GenLODs", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginGroomToMeshTool", new IMAGE_BRUSH_SVG("Icons/HairHelmet", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginGroomToMeshTool.Small", new IMAGE_BRUSH_SVG("Icons/HairHelmet", Icon20x20));
		
		StyleSet->Set("ModelingToolsManagerCommands.BeginSkeletonEditingTool", new IMAGE_BRUSH_SVG("Icons/SkeletalEditor_20", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSkinWeightsBindingTool", new IMAGE_BRUSH_SVG("Icons/BindSkin", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSkinWeightsPaintTool", new IMAGE_BRUSH_SVG("Icons/EditWeights", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSkinWeightsPaintTool.Small", new IMAGE_BRUSH_SVG("Icons/EditWeights", Icon20x20));

		StyleSet->Set("ModelingToolsManagerCommands.MeshSelectionModeAction_NoSelection", new IMAGE_BRUSH_SVG("Icons/ModSelectionObject_16", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.MeshSelectionModeAction_MeshTriangles", new IMAGE_BRUSH_SVG("Icons/ModSelectionPolys_16", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.MeshSelectionModeAction_MeshVertices", new IMAGE_BRUSH_SVG("Icons/ModSelectionVerts_16", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.MeshSelectionModeAction_MeshEdges", new IMAGE_BRUSH_SVG("Icons/ModSelectionEdges_16", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.MeshSelectionModeAction_GroupFaces", new IMAGE_BRUSH_SVG("Icons/ModSelectionPolygroupFaces_16", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.MeshSelectionModeAction_GroupCorners", new IMAGE_BRUSH_SVG("Icons/ModSelectionPolygroupVerts_16", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.MeshSelectionModeAction_GroupEdges", new IMAGE_BRUSH_SVG("Icons/ModSelectionPolygroupEdges_16", Icon20x20));

		StyleSet->Set("ModelingToolsManagerCommands.MeshElementSelection", new IMAGE_BRUSH_SVG("Icons/MeshSelect_16", Icon20x20));

		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_SelectAll", new IMAGE_BRUSH_SVG("Icons/ModSelectionAll_16", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_Invert", new IMAGE_BRUSH_SVG("Icons/ModSelectionInverse_16", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_ExpandToConnected", new IMAGE_BRUSH_SVG("Icons/ModSelectionConnected_16", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_InvertConnected", new IMAGE_BRUSH_SVG("Icons/ModSelectionConnectedInverse_16", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_Expand", new IMAGE_BRUSH_SVG("Icons/ModSelectionExpand_16", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_Contract", new IMAGE_BRUSH_SVG("Icons/ModSelectionShrink_16", Icon20x20));

		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_Delete", new IMAGE_BRUSH_SVG("Icons/Delete", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_Delete.Small", new IMAGE_BRUSH_SVG("Icons/Delete", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_Extrude", new IMAGE_BRUSH_SVG("Icons/Extrude", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_Extrude.Small", new IMAGE_BRUSH_SVG("Icons/Extrude", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_Offset", new IMAGE_BRUSH_SVG("Icons/Offset", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_Offset.Small", new IMAGE_BRUSH_SVG("Icons/Offset", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_PolyEd", new IMAGE_BRUSH_SVG("Icons/PolyEdit", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_PolyEd.Small", new IMAGE_BRUSH_SVG("Icons/PolyEdit",	Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_TriSel", new IMAGE_BRUSH_SVG("Icons/TriSelect", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_TriSel.Small", new IMAGE_BRUSH_SVG("Icons/TriSelect", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_Inset", new IMAGE_BRUSH_SVG("Icons/Inset", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_Inset.Small", new IMAGE_BRUSH_SVG("Icons/Inset", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_Outset", new IMAGE_BRUSH_SVG("Icons/Outset", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_Outset.Small", new IMAGE_BRUSH_SVG("Icons/Outset", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_CutFaces", new IMAGE_BRUSH_SVG("Icons/CutFaces", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_CutFaces.Small", new IMAGE_BRUSH_SVG("Icons/CutFaces", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_InsertEdgeLoop", new IMAGE_BRUSH_SVG("Icons/EdgeLoop", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_InsertEdgeLoop.Small", new IMAGE_BRUSH_SVG("Icons/EdgeLoop", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_ExtrudeEdges", new IMAGE_BRUSH_SVG("Icons/ExtrudeEdge", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_ExtrudeEdges.Small", new IMAGE_BRUSH_SVG("Icons/ExtrudeEdge", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_PushPull", new IMAGE_BRUSH_SVG("Icons/PushPull", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_PushPull.Small", new IMAGE_BRUSH_SVG("Icons/PushPull", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_Bevel", new IMAGE_BRUSH_SVG("Icons/Bevel", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginPolyModelTool_Bevel.Small", new IMAGE_BRUSH_SVG("Icons/Bevel", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_Retriangulate", new IMAGE_BRUSH_SVG("Icons/Clean", Icon20x20));
		StyleSet->Set("ModelingToolsManagerCommands.BeginSelectionAction_Retriangulate.Small", new IMAGE_BRUSH_SVG("Icons/Clean", Icon20x20));

		StyleSet->Set("ModelingModeSelection.More_Right",  new IMAGE_BRUSH_SVG("Icons/SelectionToolbar_More", Icon20x20));
		StyleSet->Set("ModelingModeSelection.Edits_Right",  new IMAGE_BRUSH_SVG("Icons/SelectionToolbar_Edits", Icon20x20));


		//
		// icons and style for the mesh selection toolbar
		//
		StyleSet->Set("SelectionToolBarIcons.LockedTarget", new IMAGE_BRUSH_SVG("Icons/lock-red", Icon16x16));
		StyleSet->Set("SelectionToolBarIcons.UnlockedTarget", new IMAGE_BRUSH_SVG("Icons/lock-unlocked-green", Icon16x16));

		FToolBarStyle SelectionToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("EditorViewportToolBar");
		StyleSet->Set("SelectionToolBar", SelectionToolbarStyle);

		// override red-button style
		const FButtonStyle SelectionToolbarRedButton = FButtonStyle(SelectionToolbarStyle.ButtonStyle)
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::AccentRed, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::AccentRed, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::AccentRed, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::SelectInactive, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetNormalForeground(FSlateColor::UseForeground())
			.SetPressedForeground(FSlateColor::UseForeground())
			.SetHoveredForeground(FSlateColor::UseForeground());
		SelectionToolbarStyle.SetButtonStyle(SelectionToolbarRedButton);
		StyleSet->Set("SelectionToolBar.RedButton", SelectionToolbarStyle);

		// override green-button style
		const FButtonStyle SelectionToolbarGreenButton = FButtonStyle(SelectionToolbarStyle.ButtonStyle)
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::AccentGreen, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::AccentGreen, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::AccentGreen, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::SelectInactive, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetNormalForeground(FSlateColor::UseForeground())
			.SetPressedForeground(FSlateColor::UseForeground())
			.SetHoveredForeground(FSlateColor::UseForeground());
		SelectionToolbarStyle.SetButtonStyle(SelectionToolbarGreenButton);
		StyleSet->Set("SelectionToolBar.GreenButton", SelectionToolbarStyle);

		//
		// Icons for brush falloffs in sculpt/etc tools
		//

		StyleSet->Set("BrushFalloffIcons.Smooth", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_Smooth", Icon120));
		StyleSet->Set("BrushFalloffIcons.Linear", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_Linear", Icon120));
		StyleSet->Set("BrushFalloffIcons.Inverse", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_Inverse", Icon120));
		StyleSet->Set("BrushFalloffIcons.Round", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_Round", Icon120));
		StyleSet->Set("BrushFalloffIcons.BoxSmooth", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_BoxSmooth", Icon120));
		StyleSet->Set("BrushFalloffIcons.BoxLinear", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_BoxLinear", Icon120));
		StyleSet->Set("BrushFalloffIcons.BoxInverse", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_BoxInverse", Icon120));
		StyleSet->Set("BrushFalloffIcons.BoxRound", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_BoxRound", Icon120));


		//
		// Icons for brushes in sculpt/etc tools
		//

		StyleSet->Set("BrushTypeIcons.Smooth", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Smooth", Icon120));
		StyleSet->Set("BrushTypeIcons.SmoothFill", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_SmoothFill", Icon120));
		StyleSet->Set("BrushTypeIcons.Move", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Move", Icon120));
		StyleSet->Set("BrushTypeIcons.SculptN", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_SculptN", Icon120));
		StyleSet->Set("BrushTypeIcons.SculptV", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_SculptV", Icon120));
		StyleSet->Set("BrushTypeIcons.SculptMx", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_SculptMx", Icon120));
		StyleSet->Set("BrushTypeIcons.Inflate", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Inflate", Icon120));
		StyleSet->Set("BrushTypeIcons.InflateSt", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Inflate", Icon120));
		StyleSet->Set("BrushTypeIcons.InflateMax", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Inflate", Icon120)); // TODO: Need a new icon for this one
		StyleSet->Set("BrushTypeIcons.Pinch", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Pinch", Icon120));
		StyleSet->Set("BrushTypeIcons.Flatten", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Flatten", Icon120));
		StyleSet->Set("BrushTypeIcons.PlaneN", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_PlaneN", Icon120));
		StyleSet->Set("BrushTypeIcons.PlaneV", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_PlaneV", Icon120));
		StyleSet->Set("BrushTypeIcons.PlaneW", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_PlaneW", Icon120));
		StyleSet->Set("BrushTypeIcons.Scale", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Scale", Icon120));
		StyleSet->Set("BrushTypeIcons.Grab", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Grab", Icon120));
		StyleSet->Set("BrushTypeIcons.GrabSharp", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_GrabSharp", Icon120));
		StyleSet->Set("BrushTypeIcons.Twist", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Twist", Icon120));
		StyleSet->Set("BrushTypeIcons.EraseSculptLayer", new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/Landscape/Landscape_Erase_x40.png")), Icon20x20));
		// Height brushes (currently using same icons as corresponding landscape brushes)
		StyleSet->Set("BrushTypeIcons.HeightSculpt", new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/Landscape/Landscape_Sculpt_x40.png")), Icon20x20));
		StyleSet->Set("BrushTypeIcons.HeightSmooth", new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/Landscape/Landscape_Smooth_x40.png")), Icon20x20));
		StyleSet->Set("BrushTypeIcons.HeightFlatten", new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/Landscape/Landscape_Flatten_x40.png")), Icon20x20));
		StyleSet->Set("BrushTypeIcons.HeightErase", new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/Landscape/Landscape_Erase_x40.png")), Icon20x20));

		// Pressure Sensitivity toggle button
		StyleSet->Set("BrushIcons.PressureSensitivity", new IMAGE_BRUSH_SVG("Icons/PressureSensitivity", Icon20x20));
		StyleSet->Set("BrushIcons.PressureSensitivity.Small", new IMAGE_BRUSH_SVG("Icons/PressureSensitivity", Icon16x16));

		//
		// Icons for selection buttons in PolyEd and TriEd
		//

		StyleSet->Set("PolyEd.SelectByMarquee", new FSlateVectorImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Starship/Common/MarqueeSelectOccluded_16")), Icon20x20));
		StyleSet->Set("PolyEd.SelectByLasso", new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/GeneralTools/Lasso_40x.png")), Icon20x20));

		StyleSet->Set("PolyEd.SelectCorners", new IMAGE_BRUSH_SVG("Icons/SelectionVertices", Icon20x20));
		StyleSet->Set("PolyEd.SelectEdges", new IMAGE_BRUSH_SVG("Icons/SelectionBorderEdges", Icon20x20));
		StyleSet->Set("PolyEd.SelectFaces", new IMAGE_BRUSH_SVG("Icons/SelectionTriangles3", Icon20x20));
		StyleSet->Set("PolyEd.SelectEdgeLoops", new IMAGE_BRUSH_SVG("Icons/ModelingEdgeLoopSelection", Icon20x20));
		StyleSet->Set("PolyEd.SelectEdgeRings", new IMAGE_BRUSH_SVG("Icons/ModelingEdgeRingSelection", Icon20x20));

		// Icons for PolyEd and TriEd activities
		StyleSet->Set("PolyEd.InsertGroupEdge", new IMAGE_PLUGIN_BRUSH("Icons/ModelingGroupEdgeInsert_x40", Icon20x20));
		StyleSet->Set("PolyEd.InsertEdgeLoop", new IMAGE_PLUGIN_BRUSH("Icons/ModelingEdgeLoopInsert_x40", Icon20x20));
		StyleSet->Set("PolyEd.Extrude", new IMAGE_BRUSH_SVG("Icons/Extrude", Icon20x20));
		StyleSet->Set("PolyEd.Offset", new IMAGE_BRUSH_SVG("Icons/Offset", Icon20x20));
		StyleSet->Set("PolyEd.PushPull", new IMAGE_BRUSH_SVG("Icons/PushPull", Icon20x20));
		StyleSet->Set("PolyEd.Inset", new IMAGE_BRUSH_SVG("Icons/Inset", Icon20x20));
		StyleSet->Set("PolyEd.Outset", new IMAGE_BRUSH_SVG("Icons/Outset", Icon20x20));
		StyleSet->Set("PolyEd.CutFaces", new IMAGE_BRUSH_SVG("Icons/CutFaces", Icon20x20));
		StyleSet->Set("PolyEd.ProjectUVs", new IMAGE_PLUGIN_BRUSH("Icons/UVProjection_40x", Icon20x20));
		StyleSet->Set("PolyEd.Bevel", new IMAGE_BRUSH_SVG("Icons/Bevel", Icon20x20));
		StyleSet->Set("PolyEd.ExtrudeEdge", new IMAGE_BRUSH_SVG("Icons/ExtrudeEdge", Icon20x20));
	}

	// Style for the toolbar in the PolyEd customization
	{
		// For the selection button toolbar, we want to use something similar to the toolbar we use in the viewport
		StyleSet->Set("PolyEd.SelectionToolbar", FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("EditorViewportToolBar"));

		// However, increase the size of the buttons a bit
		FCheckBoxStyle ToggleButtonStart = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.Start");
		ToggleButtonStart.SetPadding(FMargin(9, 7, 6, 7));
		StyleSet->Set("PolyEd.SelectionToolbar.ToggleButton.Start", ToggleButtonStart);

		FCheckBoxStyle ToggleButtonMiddle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.Middle");
		ToggleButtonMiddle.SetPadding(FMargin(9, 7, 6, 7));
		StyleSet->Set("PolyEd.SelectionToolbar.ToggleButton.Middle", ToggleButtonMiddle);

		FCheckBoxStyle ToggleButtonEnd = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.End");
		ToggleButtonEnd.SetPadding(FMargin(7, 7, 8, 7));
		StyleSet->Set("PolyEd.SelectionToolbar.ToggleButton.End", ToggleButtonEnd);
	}

	// Style to be applied to customizable section headers so that the color shows up properly
	{
		// look up default radii for palette toolbar expandable area headers
		FVector4 HeaderRadii(4, 4, 0, 0);
		const FSlateBrush* BaseBrush = FAppStyle::Get().GetBrush("PaletteToolbar.ExpandableAreaHeader");
		if (BaseBrush != nullptr)
		{
			HeaderRadii = BaseBrush->OutlineSettings.CornerRadii;
		}
		StyleSet->Set("ModelingMode.WhiteExpandableAreaHeader", new FSlateRoundedBoxBrush(FSlateColor(FLinearColor::White), HeaderRadii));
	}

	// Similar to EditorViewport.OverlayBrush, but opaque with a gray color to be able to be placed on top of other overlays.
	StyleSet->Set("ModelingMode.OpaqueOverlayBrush", 
		new FSlateRoundedBoxBrush(FStyleColors::Panel.GetSpecifiedColor(), 8.0, FStyleColors::Dropdown, 1.0));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef IMAGE_PLUGIN_BRUSH
#undef RootToContentDir

void FModelingToolsEditorModeStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
