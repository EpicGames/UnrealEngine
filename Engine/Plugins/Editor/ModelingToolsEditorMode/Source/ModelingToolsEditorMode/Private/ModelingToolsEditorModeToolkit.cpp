// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingToolsEditorModeToolkit.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "EditorModeManager.h"
#include "IGeometryProcessingInterfacesModule.h"
#include "GeometryProcessingInterfaces/IUVEditorModularFeature.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ModelingToolsManagerActions.h"
#include "InteractiveToolManager.h"
#include "ModelingToolsEditorModeSettings.h"
#include "ModelingToolsEditorModeStyle.h"
#include "ModelingToolsEditorMode.h"

#include "Modules/ModuleManager.h"
#include "IDetailsView.h"
#include "ISettingsModule.h"
#include "Toolkits/AssetEditorModeUILayer.h"

#include "SSimpleButton.h"
#include "STransformGizmoNumericalUIOverlay.h"
#include "Tools/UEdMode.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Input/SEditableComboBox.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "ModelingWidgets/ModelingCustomizationUtil.h"

// for Tool Extensions
#include "Features/IModularFeatures.h"
#include "ModelingModeToolExtensions.h"

// for Object Type properties
#include "PropertySets/CreateMeshObjectTypeProperties.h"

// for quick settings
#include "EditorInteractiveToolsFrameworkModule.h"
#include "Tools/EditorComponentSourceFactory.h"
#include "ToolTargetManager.h"
#include "ToolTargets/StaticMeshComponentToolTarget.h"
#include "SPrimaryButton.h"
#include "Dialogs/DlgPickPath.h"
#include "ModelingModeAssetUtils.h"
#include "ModelingToolsEditablePaletteConfig.h"

// for showing toast notifications
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"

// for presets
#include "ToolPresetAsset.h"
#include "PropertyCustomizationHelpers.h"

#include "LevelEditorViewport.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "ToolMenus.h"
#include "ToolkitBuilder.h"
#include "Dialogs/Dialogs.h"
#include "Widgets/Input/SComboButton.h"
#include "IToolPresetEditorModule.h"
#include "ToolPresetSettings.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ModelingWidgets/SToolInputAssetComboPanel.h"
#include "Fonts/SlateFontInfo.h"
#include "ToolPresetAssetSubsystem.h"
#include "Selection/GeometrySelectionManager.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "IAssetViewport.h"

// for editor selection queries
#include "Selection.h"

#if ENABLE_STYLUS_SUPPORT
#include "ModelingStylusInputHandler.h"
#endif

#define LOCTEXT_NAMESPACE "FModelingToolsEditorModeToolkit"


// if set to 1, then on mode initialization we include buttons for prototype modeling tools
static TAutoConsoleVariable<int32> CVarEnablePrototypeModelingTools(
	TEXT("modeling.EnablePrototypes"),
	0,
	TEXT("Enable unsupported Experimental prototype Modeling Tools"));
static TAutoConsoleVariable<int32> CVarEnablePolyModeling(
	TEXT("modeling.EnablePolyModel"),
	0,
	TEXT("Enable prototype PolyEdit tab"));
static TAutoConsoleVariable<int32> CVarEnableToolPresets(
	TEXT("modeling.EnablePresets"),
	1,
	TEXT("Enable tool preset features and UX"));

namespace FModelingToolsEditorModeToolkitLocals
{
	typedef TFunction<void(UInteractiveToolsPresetCollectionAsset& Preset, UInteractiveTool& Tool)> PresetAndToolFunc;
	typedef TFunction<void(UInteractiveToolsPresetCollectionAsset& Preset)> PresetOnlyFunc;

	void ExecuteWithPreset(const FSoftObjectPath& PresetPath, PresetOnlyFunc Function)
	{
		UInteractiveToolsPresetCollectionAsset* Preset = nullptr;

		UToolPresetAssetSubsystem* PresetAssetSubsystem = GEditor->GetEditorSubsystem<UToolPresetAssetSubsystem>();

		if (PresetPath.IsNull() && ensure(PresetAssetSubsystem))
		{
			Preset = PresetAssetSubsystem->GetDefaultCollection();
		}
		if (PresetPath.IsAsset())
		{
			Preset = Cast<UInteractiveToolsPresetCollectionAsset>(PresetPath.TryLoad());
		}
		if (!Preset)
		{
			return;
		}
		Function(*Preset);


	}

	void ExecuteWithPresetAndTool(UEdMode& EdMode, EToolSide ToolSide, const FSoftObjectPath& PresetPath, PresetAndToolFunc Function)
	{
		UInteractiveToolsPresetCollectionAsset* Preset = nullptr;		
		UInteractiveTool* Tool = EdMode.GetToolManager()->GetActiveTool(EToolSide::Left);

		UToolPresetAssetSubsystem* PresetAssetSubsystem = GEditor->GetEditorSubsystem<UToolPresetAssetSubsystem>();

		if (PresetPath.IsNull() && ensure(PresetAssetSubsystem))
		{
			Preset = PresetAssetSubsystem->GetDefaultCollection();
		}
		if (PresetPath.IsAsset())
		{
			Preset = Cast<UInteractiveToolsPresetCollectionAsset>(PresetPath.TryLoad());
		}
		if (!Preset || !Tool)
		{
			return;
		}
		Function(*Preset, *Tool);
	}

	void ExecuteWithPresetAndTool(UEdMode& EdMode, EToolSide ToolSide, UInteractiveToolsPresetCollectionAsset& Preset, PresetAndToolFunc Function)
	{
		UInteractiveTool* Tool = EdMode.GetToolManager()->GetActiveTool(EToolSide::Left);

		if (!Tool)
		{
			return;
		}
		Function(Preset, *Tool);
	}
}

class FRecentPresetCollectionProvider : public SToolInputAssetComboPanel::IRecentAssetsProvider
{
	public:
		//~ SToolInputAssetComboPanel::IRecentAssetsProvider interface		
		virtual TArray<FAssetData> GetRecentAssetsList() override { return RecentPresetCollectionList; }		
		virtual void NotifyNewAsset(const FAssetData& NewAsset) {
			RecentPresetCollectionList.AddUnique(NewAsset);
		};

	protected:
		TArray<FAssetData> RecentPresetCollectionList;
};

FModelingToolsEditorModeToolkit::FModelingToolsEditorModeToolkit()
{
	UModelingModeEditableToolPaletteConfig::Initialize();
	UModelingModeEditableToolPaletteConfig::Get()->LoadEditorConfig();

	UToolPresetUserSettings::Initialize();
	UToolPresetUserSettings::Get()->LoadEditorConfig();

	RecentPresetCollectionProvider = MakeShared< FRecentPresetCollectionProvider>();
	CurrentPreset = MakeShared<FAssetData>();
	
#if ENABLE_STYLUS_SUPPORT
	// Do not initialize StylusInput for headless editor instances.
	if (!GUsingNullRHI)
	{
		StylusInputHandler = MakeUnique<UE::Modeling::FStylusInputHandler>();
	}
#endif
}

FModelingToolsEditorModeToolkit::~FModelingToolsEditorModeToolkit()
{
	if (IsHosted())
	{
		if (GizmoNumericalUIOverlayWidget.IsValid())
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(GizmoNumericalUIOverlayWidget.ToSharedRef());
			GizmoNumericalUIOverlayWidget.Reset();
		}
	}
	UModelingToolsEditorModeSettings* Settings = GetMutableDefault<UModelingToolsEditorModeSettings>();
	Settings->OnModified.Remove(AssetSettingsModifiedHandle);
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolNotificationMessage.RemoveAll(this);
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolWarningMessage.RemoveAll(this);

	RecentPresetCollectionProvider = nullptr;
}


void FModelingToolsEditorModeToolkit::CustomizeModeDetailsViewArgs(FDetailsViewArgs& ArgsInOut)
{
	//ArgsInOut.ColumnWidth = 0.3f;
}


TSharedPtr<SWidget> FModelingToolsEditorModeToolkit::GetInlineContent() const
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.VAlign(VAlign_Fill)
		[
			ToolkitWidget.ToSharedRef()
		];
}

void FModelingToolsEditorModeToolkit::RegisterPalettes()
{
	// Note, currently this code path does not use short names- those only get used if 
	//  UISettings->bUseLegacyModelingPalette is true, inside BuildToolPalette. If we someday
	//  do want to use short names here, FModelingToolsManagerCommands::GetCommandLabel()
	//  can give those.

	const FModelingToolsManagerCommands& Commands = FModelingToolsManagerCommands::Get();
	const TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();

	UEdMode* ScriptableMode = GetScriptableEditorMode().Get();
	UModelingToolsModeCustomizationSettings* UISettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>();
	UModelingToolsEditorModeSettings* ModelingModeSettings = GetMutableDefault<UModelingToolsEditorModeSettings>();
	UISettings->OnSettingChanged().AddSP(SharedThis(this), &FModelingToolsEditorModeToolkit::UpdateCategoryButtonLabelVisibility);
	
	ToolkitSections = MakeShared<FToolkitSections>();
	FToolkitBuilderArgs ToolkitBuilderArgs(ScriptableMode->GetModeInfo().ToolbarCustomizationName);
	ToolkitBuilderArgs.ToolkitCommandList = GetToolkitCommands();
	ToolkitBuilderArgs.ToolkitSections = ToolkitSections;
	ToolkitBuilderArgs.SelectedCategoryTitleVisibility = EVisibility::Collapsed;
	// This lets us re-show the buttons if the user clicks a category with a tool still active.
	ToolkitBuilderArgs.CategoryReclickBehavior = FToolkitBuilder::ECategoryReclickBehavior::TreatAsChanged;
	ToolkitBuilder = MakeShared<FToolkitBuilder>(ToolkitBuilderArgs);
	ToolkitBuilder->SetCategoryButtonLabelVisibility(UISettings->bShowCategoryButtonLabels);

	// We need to specify the modeling mode specific config instance because this cannot exist at the base toolkit level due to dependency issues currently
	FGetEditableToolPaletteConfigManager GetConfigManager = FGetEditableToolPaletteConfigManager::CreateStatic(&UModelingModeEditableToolPaletteConfig::GetAsConfigManager);

	FavoritesPalette =
		MakeShareable(new FEditablePalette(
			Commands.LoadFavoritesTools.ToSharedRef(),
			Commands.AddToFavorites.ToSharedRef(),
			Commands.RemoveFromFavorites.ToSharedRef(),
			"ModelingToolsEditorModeFavoritesPalette",
			GetConfigManager));
	
	ToolkitBuilder->AddPalette(StaticCastSharedPtr<FEditablePalette>(FavoritesPalette));


	TArray<TSharedPtr<FUICommandInfo>> CreatePaletteItems({
		Commands.BeginAddBoxPrimitiveTool,
		Commands.BeginAddSpherePrimitiveTool,
		Commands.BeginAddCylinderPrimitiveTool,
		Commands.BeginAddCapsulePrimitiveTool,
		Commands.BeginAddConePrimitiveTool,
		Commands.BeginAddTorusPrimitiveTool,
		Commands.BeginAddArrowPrimitiveTool,
		Commands.BeginAddRectanglePrimitiveTool,
		Commands.BeginAddDiscPrimitiveTool,
		Commands.BeginAddStairsPrimitiveTool,
		Commands.BeginCubeGridTool,

		Commands.BeginDrawPolygonTool,
		Commands.BeginDrawPolyPathTool,
		Commands.BeginDrawAndRevolveTool,
		Commands.BeginRevolveSplineTool,
		Commands.BeginDrawSplineTool,
		Commands.BeginExtractSplineTool,
		Commands.BeginTriangulateSplinesTool
	});
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadCreateTools.ToSharedRef(), CreatePaletteItems ) ) );

	if ( ModelingModeSettings && ModelingModeSettings->GetMeshSelectionsEnabled() )
	{
		TArray<TSharedPtr<FUICommandInfo>> SelectionPaletteItems({
			Commands.BeginSelectionAction_Delete,
			Commands.BeginSelectionAction_Extrude,
			Commands.BeginSelectionAction_Offset,
			Commands.BeginPolyModelTool_ExtrudeEdges,
			Commands.BeginPolyModelTool_PushPull,

			Commands.BeginPolyModelTool_Inset,
			Commands.BeginPolyModelTool_Outset,
			Commands.BeginPolyModelTool_CutFaces,
			Commands.BeginPolyModelTool_Bevel,

			Commands.BeginPolyModelTool_InsertEdgeLoop,
			Commands.BeginSelectionAction_Retriangulate,

			Commands.BeginPolyModelTool_PolyEd,
			Commands.BeginPolyModelTool_TriSel
		});
		ToolkitBuilder->AddPalette( 
			MakeShareable( new FToolPalette( Commands.LoadSelectionTools.ToSharedRef(), SelectionPaletteItems ) ) );
	}


	TArray<TSharedPtr<FUICommandInfo>> TransformPaletteItems({
		Commands.BeginTransformMeshesTool,
		Commands.BeginAlignObjectsTool,

		Commands.BeginCombineMeshesTool,
		Commands.BeginDuplicateMeshesTool,

		Commands.BeginEditPivotTool,
		Commands.BeginBakeTransformTool,

		Commands.BeginTransferMeshTool,
		Commands.BeginConvertMeshesTool,

		Commands.BeginSplitMeshesTool,
		Commands.BeginPatternTool,
		
		Commands.BeginHarvestInstancesTool,
		Commands.BeginISMEditorTool
	});
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadTransformTools.ToSharedRef(), TransformPaletteItems ) ) );



	TArray<TSharedPtr<FUICommandInfo>> DeformPaletteItems({
		Commands.BeginSculptMeshTool,
		Commands.BeginRemeshSculptMeshTool,
		Commands.BeginSmoothMeshTool,
		Commands.BeginOffsetMeshTool,
		Commands.BeginMeshSpaceDeformerTool,
		Commands.BeginLatticeDeformerTool,
		Commands.BeginDisplaceMeshTool,
		Commands.BeginPolyDeformTool
	});
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadDeformTools.ToSharedRef(), DeformPaletteItems ) ) );
	


	TArray<TSharedPtr<FUICommandInfo>> PolyToolsPaletteItems({
		Commands.BeginPolyEditTool,
		Commands.BeginSubdividePolyTool,

		Commands.BeginMeshBooleanTool,
		Commands.BeginPolygonCutTool,

		Commands.BeginPlaneCutTool,
		Commands.BeginMirrorTool,

		Commands.BeginCutMeshWithMeshTool,
		Commands.BeginMeshTrimTool

		});
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadPolyTools.ToSharedRef(), PolyToolsPaletteItems ) ) );



	TArray<TSharedPtr<FUICommandInfo>> TriToolsPaletteItems({
		Commands.BeginMeshSelectionTool,
		Commands.BeginTriEditTool,

		Commands.BeginHoleFillTool,
		Commands.BeginWeldEdgesTool,

		Commands.BeginSelfUnionTool,
		Commands.BeginRemoveOccludedTrianglesTool,

		Commands.BeginSimplifyMeshTool,
		Commands.BeginRemeshMeshTool,

		Commands.BeginProjectToTargetTool,
		Commands.BeginMeshInspectorTool
		});
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadMeshOpsTools.ToSharedRef(), TriToolsPaletteItems ) ) );



	TArray<TSharedPtr<FUICommandInfo>> VoxelPaletteItems({
		Commands.BeginVoxelSolidifyTool,
		Commands.BeginVoxelBlendTool,
		Commands.BeginVoxelMorphologyTool
	});
#if WITH_PROXYLOD
	VoxelPaletteItems.Add(Commands.BeginVoxelBooleanTool);
	VoxelPaletteItems.Add(Commands.BeginVoxelMergeTool);
#endif	// WITH_PROXYLOD
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadVoxOpsTools.ToSharedRef(), VoxelPaletteItems ) ) );



	TArray<TSharedPtr<FUICommandInfo>> BakingPaletteItems({
		Commands.BeginBakeMeshAttributeMapsTool,
		Commands.BeginBakeMultiMeshAttributeMapsTool,
		Commands.BeginBakeMeshAttributeVertexTool,
		Commands.BeginBakeRenderCaptureTool
	});
	BakingPaletteItems.Add(Commands.BeginMeshInspectorTool);
	BakingPaletteItems.Add(Commands.BeginEditTangentsTool);
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadBakingTools.ToSharedRef(), BakingPaletteItems ) ) );



	TArray<TSharedPtr<FUICommandInfo>> UVPaletteItems({
		Commands.BeginGlobalUVGenerateTool,
		Commands.BeginGroupUVGenerateTool,
		Commands.BeginUVProjectionTool,
		Commands.BeginUVSeamEditTool,
		Commands.BeginTransformUVIslandsTool,
		Commands.BeginUVLayoutTool,
		Commands.BeginUVTransferTool
	});
	if (IModularFeatures::Get().IsModularFeatureAvailable(IUVEditorModularFeature::GetModularFeatureName()))
	{
		UVPaletteItems.Add(Commands.LaunchUVEditor);
	}
	UVPaletteItems.Add(Commands.BeginMeshInspectorTool);
	UVPaletteItems.Add(Commands.BeginMeshGroupPaintTool);

	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadUVsTools.ToSharedRef(), UVPaletteItems)) );



	TArray<TSharedPtr<FUICommandInfo>> AttributesToolsPaletteItems({
		Commands.BeginMeshInspectorTool,
		Commands.BeginLODManagerTool,

		Commands.BeginEditNormalsTool,
		Commands.BeginEditTangentsTool,

		Commands.BeginPolyGroupsTool,
		Commands.BeginMeshGroupPaintTool,

		Commands.BeginAttributeEditorTool,
		Commands.BeginEditMeshMaterialsTool,

		Commands.BeginMeshVertexPaintTool,
		Commands.BeginMeshAttributePaintTool,

		Commands.BeginPhysicsInspectorTool,
		Commands.BeginSimpleCollisionEditorTool,
		Commands.BeginSetCollisionGeometryTool,
		Commands.BeginExtractCollisionGeometryTool
	});
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadAttributesTools.ToSharedRef(), AttributesToolsPaletteItems ) ) );



	TArray<TSharedPtr<FUICommandInfo>> LODPaletteItems({
		Commands.BeginGenerateStaticMeshLODAssetTool,
		Commands.BeginAddPivotActorTool,
		Commands.BeginRevolveBoundaryTool,
		Commands.BeginVolumeToMeshTool,
		Commands.BeginMeshToVolumeTool
	});
	if (Commands.BeginBspConversionTool)
	{
		LODPaletteItems.Add(Commands.BeginBspConversionTool);
	}
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadLodsTools.ToSharedRef(), LODPaletteItems)) );

	ToolkitBuilder->SetActivePaletteOnLoad(Commands.LoadCreateTools.Get());
	ToolkitBuilder->UpdateWidget();


	if (IModularFeatures::Get().IsModularFeatureAvailable(IModelingModeToolExtension::GetModularFeatureName()))
	{
		TArray<IModelingModeToolExtension*> Extensions = IModularFeatures::Get().GetModularFeatureImplementations<IModelingModeToolExtension>(
			IModelingModeToolExtension::GetModularFeatureName());
		for (int32 k = 0; k < Extensions.Num(); ++k)
		{
			FText ExtensionName = Extensions[k]->GetExtensionName();
			FText SectionName = Extensions[k]->GetToolSectionName();

			FModelingModeExtensionExtendedInfo ExtensionExtendedInfo;
			bool bHasExtendedInfo = Extensions[k]->GetExtensionExtendedInfo(ExtensionExtendedInfo);

			TSharedPtr<FUICommandInfo> PaletteCommand;
			if (bHasExtendedInfo && ExtensionExtendedInfo.ExtensionCommand.IsValid())
			{
				PaletteCommand = ExtensionExtendedInfo.ExtensionCommand;
			}
			else
			{
				FText UseTooltipText = (bHasExtendedInfo && ExtensionExtendedInfo.ToolPaletteButtonTooltip.IsEmpty() == false) ?
					ExtensionExtendedInfo.ToolPaletteButtonTooltip : SectionName;
				PaletteCommand = FModelingToolsManagerCommands::RegisterExtensionPaletteCommand(
					FName(ExtensionName.ToString()),
					SectionName, UseTooltipText, FSlateIcon());
			}

			TArray<TSharedPtr<FUICommandInfo>> PaletteItems;
			FExtensionToolQueryInfo ExtensionQueryInfo;
			ExtensionQueryInfo.bIsInfoQueryOnly = true;
			TArray<FExtensionToolDescription> ToolSet;
			Extensions[k]->GetExtensionTools(ExtensionQueryInfo, ToolSet);
			for (const FExtensionToolDescription& ToolInfo : ToolSet)
			{
				PaletteItems.Add(ToolInfo.ToolCommand);
			}

			ToolkitBuilder->AddPalette(
				MakeShareable(new FToolPalette(PaletteCommand.ToSharedRef(), PaletteItems)));
		}
	}



	// if selected palette changes, make sure we are showing the palette command buttons, which may be hidden by active Tool
	ActivePaletteChangedHandle = ToolkitBuilder->OnActivePaletteChanged.AddLambda([this]()
	{
		ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Visible);
	});
}







void FModelingToolsEditorModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	const UModelingToolsModeCustomizationSettings* UISettings = GetDefault<UModelingToolsModeCustomizationSettings>();
	bUsesToolkitBuilder = !UISettings->bUseLegacyModelingPalette;
	
	// Have to create the ToolkitWidget here because FModeToolkit::Init() is going to ask for it and add
	// it to the Mode panel, and not ask again afterwards. However we have to call Init() to get the 
	// ModeDetailsView created, that we need to add to the ToolkitWidget. So, we will create the Widget
	// here but only add the rows to it after we call Init()

	const TSharedPtr<SVerticalBox> ToolkitWidgetVBox = SNew(SVerticalBox);
	
	if ( !bUsesToolkitBuilder )
	{
		SAssignNew(ToolkitWidget, SBorder)
			.HAlign(HAlign_Fill)
			.Padding(4)
			[
				ToolkitWidgetVBox->AsShared()
			];
	}

	FModeToolkit::Init(InitToolkitHost, InOwningMode);

	GetToolkitHost()->OnActiveViewportChanged().AddSP(this, &FModelingToolsEditorModeToolkit::OnActiveViewportChanged);

	ModeWarningArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
	ModeWarningArea->SetText(FText::GetEmpty());
	ModeWarningArea->SetVisibility(EVisibility::Collapsed);

	ModeHeaderArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 12));
	ModeHeaderArea->SetText(LOCTEXT("SelectToolLabel", "Select a Tool from the Toolbar"));
	ModeHeaderArea->SetJustification(ETextJustify::Center);


	ToolWarningArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
	ToolWarningArea->SetText(FText::GetEmpty());

	ToolPresetArea = MakePresetPanel();

	// add the various sections to the mode toolbox
	ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
		[
			ToolPresetArea->AsShared()
		];
	ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
		[
			ModeWarningArea->AsShared()
		];

	if (bUsesToolkitBuilder)
	{
		RegisterPalettes();
	}
	else
	{
		ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
			[
				ModeHeaderArea->AsShared()
			];
		ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
		[
			ToolWarningArea->AsShared()
		];	
		ToolkitWidgetVBox->AddSlot().HAlign(HAlign_Fill).FillHeight(1.f)
		[
			ModeDetailsView->AsShared()
		];
		ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).VAlign(VAlign_Bottom).Padding(5)
		[
		MakeAssetConfigPanel()->AsShared()];
	}
		

	ClearNotification();
	ClearWarning();

	if (HasToolkitBuilder())
	{
		ToolkitSections->ModeWarningArea = ModeWarningArea;
		ToolkitSections->ToolPresetArea = ToolPresetArea;
		ToolkitSections->DetailsView = ModeDetailsView;
		ToolkitSections->ToolWarningArea = ToolWarningArea;
		ToolkitSections->Footer = MakeAssetConfigPanel();

		SAssignNew(ToolkitWidget, SBorder)
		.HAlign(HAlign_Fill)
		.Padding(0)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			ToolkitBuilder->GenerateWidget()->AsShared()
		];	
	}

	ActiveToolName = FText::GetEmpty();
	ActiveToolMessage = FText::GetEmpty();

	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolNotificationMessage.AddSP(this, &FModelingToolsEditorModeToolkit::PostNotification);
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolWarningMessage.AddSP(this, &FModelingToolsEditorModeToolkit::PostWarning);

	UpdateObjectCreationOptionsFromSettings();

	MakeToolShutdownOverlayWidget();

	// Note that the numerical UI widget should be created before making the selection palette so that
	// it can be bound to the buttons there.
	MakeGizmoNumericalUIOverlayWidget();
	GetToolkitHost()->AddViewportOverlayWidget(GizmoNumericalUIOverlayWidget.ToSharedRef());

	CurrentPresetPath = FSoftObjectPath(); // Default to the default collection by leaving this null.

	UModelingToolsModeCustomizationSettings* ModelingCustomizationSettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>();
	ModelingCustomizationSettings->OnSettingChanged().AddSP(SharedThis(this),  &FModelingToolsEditorModeToolkit::UpdateSelectionColors);

#if ENABLE_STYLUS_SUPPORT
	if (StylusInputHandler && InitToolkitHost)
	{
		StylusInputHandler->RegisterWindow(InitToolkitHost->GetParentWidget());
	}
#endif
}

void FModelingToolsEditorModeToolkit::MakeToolShutdownOverlayWidget()
{
	const FSlateBrush* OverlayBrush = FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush");
	// If there is another mode, it might also have an overlay, and we would like ours to be opaque in that case
	// to draw on top cleanly (e.g., level instance editing mode has an overlay in the same place. Note that level
	// instance mode currently marks itself as not visible despite the overlay, so we shouldn't use IsOnlyVisibleActiveMode)
	if (!GetEditorModeManager().IsOnlyActiveMode(UModelingToolsEditorMode::EM_ModelingToolsEditorModeId))
	{
		OverlayBrush = FModelingToolsEditorModeStyle::Get()->GetBrush("ModelingMode.OpaqueOverlayBrush");
	}

	// Helpers to determine button/label visibility based on overrides
	auto GetSubActionIcon = [this]() -> const FSlateBrush*
	{
		if (AcceptCancelButtonParams.IsSet())
		{
			if (AcceptCancelButtonParams->IconName.IsSet())
			{
				return FModelingToolsEditorModeStyle::Get()->GetOptionalBrush(AcceptCancelButtonParams->IconName.GetValue(), nullptr, nullptr);
			}
		}
		else if (CompleteButtonParams.IsSet() && CompleteButtonParams->IconName.IsSet())
		{
			return FModelingToolsEditorModeStyle::Get()->GetOptionalBrush(CompleteButtonParams->IconName.GetValue(), nullptr, nullptr);
		}
		return nullptr;
	};
	auto GetSubActionIconVisibility = [this]()
	{
		if (AcceptCancelButtonParams.IsSet())
		{
			if (AcceptCancelButtonParams->IconName.IsSet() 
				&& FModelingToolsEditorModeStyle::Get()->GetOptionalBrush(AcceptCancelButtonParams->IconName.GetValue(), nullptr, nullptr))
			{
				return EVisibility::Visible;
			}
		}
		else if (CompleteButtonParams.IsSet() && CompleteButtonParams->IconName.IsSet()
			&& FModelingToolsEditorModeStyle::Get()->GetOptionalBrush(CompleteButtonParams->IconName.GetValue(), nullptr, nullptr))
		{
			return EVisibility::Visible;
		}
		return EVisibility::Collapsed;
	};
	auto GetSubActionLabel = [this]()
	{
		return AcceptCancelButtonParams.IsSet() ? AcceptCancelButtonParams->Label
			: CompleteButtonParams.IsSet() ? CompleteButtonParams->Label
			: FText::GetEmpty();
	};
	auto GetSubActionLabelVisibility = [this]()
	{
		return (AcceptCancelButtonParams.IsSet() || CompleteButtonParams.IsSet()) ?
			EVisibility::Visible : EVisibility::Collapsed;
	};
	auto GetAcceptButtonText = [this]()
	{
		return AcceptCancelButtonParams.IsSet() && AcceptCancelButtonParams->OverrideAcceptButtonText.IsSet() ?
			AcceptCancelButtonParams->OverrideAcceptButtonText.GetValue() : LOCTEXT("OverlayAccept", "Accept");
	};
	auto GetAcceptButtonTooltip = [this]()
	{
		return AcceptCancelButtonParams.IsSet() && AcceptCancelButtonParams->OverrideAcceptButtonTooltip.IsSet() ?
			AcceptCancelButtonParams->OverrideAcceptButtonTooltip.GetValue()
			: LOCTEXT("OverlayAcceptTooltip", "Accept/Commit the results of the active Tool [Enter]");
	};
	auto GetAcceptButtonEnabled = [this]()
	{
		return AcceptCancelButtonParams.IsSet() ? AcceptCancelButtonParams->CanAccept()
			: GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanAcceptActiveTool();
	};
	auto GetAcceptCancelButtonVisibility = [this]()
	{
		if (AcceptCancelButtonParams.IsSet()
			|| (!CompleteButtonParams.IsSet()
				&& GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->ActiveToolHasAccept()))
		{
			return EVisibility::Visible;
		}
		return EVisibility::Collapsed;
	};
	auto GetCancelButtonText = [this]()
	{
		return AcceptCancelButtonParams.IsSet() && AcceptCancelButtonParams->OverrideCancelButtonText.IsSet() ?
			AcceptCancelButtonParams->OverrideCancelButtonText.GetValue() : LOCTEXT("OverlayCancel", "Cancel");
	};
	auto GetCancelButtonTooltip = [this]()
	{
		return AcceptCancelButtonParams.IsSet() && AcceptCancelButtonParams->OverrideCancelButtonTooltip.IsSet() ?
			AcceptCancelButtonParams->OverrideCancelButtonTooltip.GetValue()
			: LOCTEXT("OverlayCancelTooltip", "Cancel the active Tool [Esc]");
	};
	auto GetCancelButtonEnabled = [this]() 
	{
		return AcceptCancelButtonParams.IsSet()
			|| GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanCancelActiveTool();
	};
	auto GetCompleteButtonText = [this]()
	{
		return CompleteButtonParams.IsSet() && CompleteButtonParams->OverrideCompleteButtonText.IsSet() ?
			CompleteButtonParams->OverrideCompleteButtonText.GetValue() : LOCTEXT("OverlayComplete", "Complete");
	};
	auto GetCompleteButtonTooltip = [this]()
	{
		return CompleteButtonParams.IsSet() && CompleteButtonParams->OverrideCompleteButtonTooltip.IsSet() ?
			CompleteButtonParams->OverrideCompleteButtonTooltip.GetValue()
			: LOCTEXT("OverlayCompleteTooltip", "Exit the active Tool [Enter]");
	};
	auto GetCompleteButtonEnabled = [this]()
	{
		return CompleteButtonParams.IsSet()
			|| GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanCompleteActiveTool();
	};
	auto GetCompleteButtonVisibility = [this]()
	{
		if (CompleteButtonParams.IsSet()
			|| (!AcceptCancelButtonParams.IsSet()
				&& GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanCompleteActiveTool()))
		{
			return EVisibility::Visible;
		}
		return EVisibility::Collapsed;
	};

	SAssignNew(ToolShutdownViewportOverlayWidget, SHorizontalBox)

	+SHorizontalBox::Slot()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Bottom)
	.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
	[
		SNew(SBorder)
		.BorderImage(OverlayBrush)
		.Padding(8.f)
		[
			SNew(SHorizontalBox)

			// Tool icon and name
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda([this] () { return ActiveToolIcon; })
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(STextBlock)
				.Text(this, &FModelingToolsEditorModeToolkit::GetActiveToolDisplayName)
			]

			// Optional: "-> [icon] SubtoolAction"
			// arrow
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(FMargin(0., 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image(FModelingToolsEditorModeStyle::Get()->GetBrush("ModelingMode.SubToolArrow"))
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Visibility_Lambda(GetSubActionLabelVisibility)
			]
			// subaction icon
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda(GetSubActionIcon)
				.Visibility_Lambda(GetSubActionIconVisibility)
			]
			// subaction label
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(STextBlock)
				.Text_Lambda(GetSubActionLabel)
				.Visibility_Lambda(GetSubActionLabelVisibility)
			]

			// Buttons:
			// Accept
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.0, 0.f, 2.f, 0.f))
			[
				SNew(SPrimaryButton)
				.Text_Lambda(GetAcceptButtonText)
				.ToolTipText_Lambda(GetAcceptButtonTooltip)
				.OnClicked_Raw(this, &FModelingToolsEditorModeToolkit::HandleAcceptCancelClick, true)
				.IsEnabled_Lambda(GetAcceptButtonEnabled)
				.Visibility_Lambda(GetAcceptCancelButtonVisibility)
			]
			// Cancel
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.Text_Lambda(GetCancelButtonText)
				.ToolTipText_Lambda(GetCancelButtonTooltip)
				.HAlign(HAlign_Center)
				.OnClicked_Raw(this, &FModelingToolsEditorModeToolkit::HandleAcceptCancelClick, false)
				.IsEnabled_Lambda(GetCancelButtonEnabled)
				.Visibility_Lambda(GetAcceptCancelButtonVisibility)
			]
			// Complete
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SPrimaryButton)
				.Text_Lambda(GetCompleteButtonText)
				.ToolTipText_Lambda(GetCompleteButtonTooltip)
				.OnClicked_Raw(this, &FModelingToolsEditorModeToolkit::HandleCompleteClick)
				.IsEnabled_Lambda(GetCompleteButtonEnabled)
				.Visibility_Lambda(GetCompleteButtonVisibility)
			]
		]	
	];

}


void FModelingToolsEditorModeToolkit::MakeGizmoNumericalUIOverlayWidget()
{
	GizmoNumericalUIOverlayWidget = SNew(STransformGizmoNumericalUIOverlay)
		.DefaultLeftPadding(15)
		// Position above the little axis visualization
		.DefaultVerticalPadding(75)
		.bPositionRelativeToBottom(true);
}


TSharedPtr<SWidget> FModelingToolsEditorModeToolkit::MakeAssetConfigPanel()
{
	//
	// New Asset Location drop-down
	//

	AssetLocationModes.Add(MakeShared<FString>(LOCTEXT("AssetLocationModeAutoGenWorldRelative", "AutoGen Folder (World-Relative)").ToString()));
	AssetLocationModes.Add(MakeShared<FString>(LOCTEXT("AssetLocationModeAutoGenGlobal", "AutoGen Folder (Global)").ToString()));
	AssetLocationModes.Add(MakeShared<FString>(LOCTEXT("AssetLocationModeCurrentFolder", "Current Folder").ToString()));
	AssetLocationMode = SNew(STextComboBox)
		.OptionsSource(&AssetLocationModes)
		.OnSelectionChanged_Lambda([&](TSharedPtr<FString> String, ESelectInfo::Type) { UpdateAssetLocationMode(String); });
	AssetSaveModes.Add(MakeShared<FString>(LOCTEXT("AssetSaveModeAutoSave", "AutoSave New Assets").ToString()));
	AssetSaveModes.Add(MakeShared<FString>(LOCTEXT("AssetSaveModeManualSave", "Manual Save").ToString()));
	AssetSaveModes.Add(MakeShared<FString>(LOCTEXT("AssetSaveModeInteractive", "Interactive").ToString()));
	AssetSaveMode = SNew(STextComboBox)
		.OptionsSource(&AssetSaveModes)
		.OnSelectionChanged_Lambda([&](TSharedPtr<FString> String, ESelectInfo::Type) { UpdateAssetSaveMode(String); });
	
	// initialize combos
	UpdateAssetPanelFromSettings();
	
	// register callback
	UModelingToolsEditorModeSettings* Settings = GetMutableDefault<UModelingToolsEditorModeSettings>();
	AssetSettingsModifiedHandle = Settings->OnModified.AddLambda([this](UObject*, FProperty*) { OnAssetSettingsModified(); });


	//
	// LOD selection dropdown
	//

	AssetLODModes.Add(MakeShared<FString>(LOCTEXT("AssetLODModeMaxAvailable", "Max Available").ToString()));
	AssetLODModes.Add(MakeShared<FString>(LOCTEXT("AssetLODModeHiRes", "HiRes").ToString()));
	AssetLODModes.Add(MakeShared<FString>(LOCTEXT("AssetLODModeLOD0", "LOD0").ToString()));
	AssetLODModes.Add(MakeShared<FString>(LOCTEXT("AssetLODModeLOD1", "LOD1").ToString()));
	AssetLODModes.Add(MakeShared<FString>(LOCTEXT("AssetLODModeLOD2", "LOD2").ToString()));
	AssetLODModes.Add(MakeShared<FString>(LOCTEXT("AssetLODModeLOD3", "LOD3").ToString()));
	AssetLODModes.Add(MakeShared<FString>(LOCTEXT("AssetLODModeLOD4", "LOD4").ToString()));
	AssetLODModes.Add(MakeShared<FString>(LOCTEXT("AssetLODModeLOD5", "LOD5").ToString()));
	AssetLODModes.Add(MakeShared<FString>(LOCTEXT("AssetLODModeLOD6", "LOD6").ToString()));
	AssetLODModes.Add(MakeShared<FString>(LOCTEXT("AssetLODModeLOD7", "LOD7").ToString()));
	AssetLODMode = SNew(STextComboBox)
		.OptionsSource(&AssetLODModes)
		.OnSelectionChanged_Lambda([&](TSharedPtr<FString> String, ESelectInfo::Type)
	{
		EMeshLODIdentifier NewSelectedLOD = EMeshLODIdentifier::LOD0;
		if (*String == *AssetLODModes[0])
		{
			NewSelectedLOD = EMeshLODIdentifier::MaxQuality;
		}
		else if (*String == *AssetLODModes[1])
		{
			NewSelectedLOD = EMeshLODIdentifier::HiResSource;
		}
		else
		{
			for (int32 k = 2; k < AssetLODModes.Num(); ++k)
			{
				if (*String == *AssetLODModes[k])
				{
					NewSelectedLOD = (EMeshLODIdentifier)(k - 2);
					break;
				}
			}
		}

		if (FEditorInteractiveToolsFrameworkGlobals::RegisteredStaticMeshTargetFactoryKey >= 0)
		{
			FComponentTargetFactory* Factory = FindComponentTargetFactoryByKey(FEditorInteractiveToolsFrameworkGlobals::RegisteredStaticMeshTargetFactoryKey);
			if (Factory != nullptr)
			{
				FStaticMeshComponentTargetFactory* StaticMeshFactory = static_cast<FStaticMeshComponentTargetFactory*>(Factory);
				StaticMeshFactory->CurrentEditingLOD = NewSelectedLOD;
			}
		}

		TObjectPtr<UToolTargetManager> TargetManager = GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->TargetManager;
		UStaticMeshComponentToolTargetFactory* StaticMeshTargetFactory = TargetManager->FindFirstFactoryByType<UStaticMeshComponentToolTargetFactory>();
		if (StaticMeshTargetFactory)
		{
			StaticMeshTargetFactory->SetActiveEditingLOD(NewSelectedLOD);
		}

	});

	AssetLODModeLabel = SNew(STextBlock)
	.Text(LOCTEXT("ActiveLODLabel", "Editing LOD"))
	.ToolTipText(LOCTEXT("ActiveLODLabelToolTip", "Select the LOD to be used when editing an existing mesh."));

	const TSharedPtr<SVerticalBox> Content = SNew(SVerticalBox)
	+ SVerticalBox::Slot().HAlign(HAlign_Fill)
	.Padding(0, 8, 0, 4)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().Padding(0).HAlign(HAlign_Left).VAlign(VAlign_Center).FillWidth(2.f)
				[ AssetLODModeLabel->AsShared() ]
			+ SHorizontalBox::Slot().Padding(0).HAlign(HAlign_Fill).Padding(0).FillWidth(4.f)
				[ AssetLODMode->AsShared() ]
		];

	SHorizontalBox::FArguments AssetOptionsArgs;
	if (Settings->InRestrictiveMode())
	{
		NewAssetPath = SNew(SEditableTextBox)
		.Text(this, &FModelingToolsEditorModeToolkit::GetRestrictiveModeAutoGeneratedAssetPathText)
		.OnTextCommitted(this, &FModelingToolsEditorModeToolkit::OnRestrictiveModeAutoGeneratedAssetPathTextCommitted);

		AssetOptionsArgs
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.Padding(0, 0, 0, 0)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().Padding(0).HAlign(HAlign_Left).VAlign(VAlign_Center).FillWidth(2.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NewAssetPathLabel", "New Asset Path"))
				.ToolTipText(LOCTEXT("NewAssetPathToolTip", "Path used for storing newly generated assets within the project folder."))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill).
			FillWidth(4.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().HAlign(HAlign_Fill).Padding(0).FillWidth(9.0f)
					[
						NewAssetPath->AsShared()
					]
					+ SHorizontalBox::Slot().HAlign(HAlign_Right).Padding(0, 0, 0, 0).AutoWidth()
					[
						SNew(SSimpleButton)
						.OnClicked_Lambda( [this]() { SelectNewAssetPath(); return FReply::Handled(); } )
						.Icon(FAppStyle::Get().GetBrush("Icons.FolderOpen"))
					]
			]

		];
	}
	else
	{
		AssetOptionsArgs
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.Padding(0, 0, 0, 0)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().HAlign(HAlign_Left).VAlign(VAlign_Center).FillWidth(2.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AssetLocationLabel", "New Asset Location"))
			]
			+ SHorizontalBox::Slot().HAlign(HAlign_Fill).FillWidth(4.f)
			[
				AssetLocationMode->AsShared()
			]
		];
	}

	Content->AddSlot()[
		SArgumentNew(AssetOptionsArgs, SHorizontalBox)
	];

	TSharedPtr<SExpandableArea> AssetConfigPanel = SNew(SExpandableArea)
		.HeaderPadding(FMargin(0.f))
		.Padding(FMargin(8.f))
		.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
		.AreaTitleFont(FAppStyle::Get().GetFontStyle("EditorModesPanel.CategoryFontStyle"))
		.BodyContent()
		[
			Content->AsShared()
		]
		.HeaderContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().HAlign(HAlign_Left).VAlign(VAlign_Center).FillWidth(2.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ModelingSettingsPanelHeader", "Modeling Mode Quick Settings"))
					
				]

				+ SHorizontalBox::Slot().HAlign(HAlign_Right).VAlign(VAlign_Center).AutoWidth()
				[
					SNew(SComboButton)
						.HasDownArrow(false)
						.MenuPlacement(EMenuPlacement::MenuPlacement_MenuRight)
						.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
						.OnGetMenuContent(FOnGetContent::CreateLambda([this]()
						{
							return MakeMenu_ModelingModeConfigSettings();
						}))
						.ContentPadding(FMargin(3.0f, 1.0f))
						.ButtonContent()
						[
							SNew(SImage)
							.Image(FModelingToolsEditorModeStyle::Get()->GetBrush("ModelingMode.DefaultSettings"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
				]
				
		];

	return AssetConfigPanel;

}

TSharedRef<SWidget> FModelingToolsEditorModeToolkit::MakePresetComboWidget(TSharedPtr<FString> InItem)
{
	return
		SNew(STextBlock)
		.Text(FText::FromString(*InItem));
}

TSharedPtr<SWidget> FModelingToolsEditorModeToolkit::MakePresetPanel()
{
	const TSharedPtr<SVerticalBox> Content = SNew(SVerticalBox);
	UModelingToolsEditorModeSettings* Settings = GetMutableDefault<UModelingToolsEditorModeSettings>();

	bool bEnableToolPresets = (CVarEnableToolPresets.GetValueOnGameThread() > 0);	
	if (!bEnableToolPresets || Settings->InRestrictiveMode())
	{
		return SNew(SVerticalBox);
	}

	const TSharedPtr<SHorizontalBox> NewContent = SNew(SHorizontalBox);
	
	auto IsToolActive = [this]() {
		if (this->OwningEditorMode.IsValid())
		{
			return this->OwningEditorMode->GetToolManager()->GetActiveTool(EToolSide::Left) != nullptr ? EVisibility::Visible : EVisibility::Collapsed;
		}
		return EVisibility::Collapsed;
	};

	NewContent->AddSlot().HAlign(HAlign_Right)
	[
		SNew(SComboButton)
		.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
		.OnGetMenuContent(this, &FModelingToolsEditorModeToolkit::GetPresetCreateButtonContent)
		.HasDownArrow(true)
		.Visibility_Lambda(IsToolActive)
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ModelingPresetPanelHeader", "Presets"))
				.Justification(ETextJustify::Center)
				.Font(FAppStyle::Get().GetFontStyle("EditorModesPanel.CategoryFontStyle"))
			]

		]
	];

	TSharedPtr<SHorizontalBox> AssetConfigPanel = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			NewContent->AsShared()
		];

	return AssetConfigPanel;
}

bool FModelingToolsEditorModeToolkit::IsPresetEnabled() const
{
	return CurrentPresetPath.IsAsset();
}


TSharedRef<SWidget> FModelingToolsEditorModeToolkit::GetPresetCreateButtonContent()
{
	RebuildPresetListForTool(false);

		auto OpenNewPresetDialog = [this]()
	{
		NewPresetLabel.Empty();
		NewPresetTooltip.Empty();

		// Set the result if they just click Ok
		SGenericDialogWidget::FArguments FolderDialogArguments;
		FolderDialogArguments.OnOkPressed_Lambda([this]()
			{
				CreateNewPresetInCollection(NewPresetLabel,
					CurrentPresetPath,
					NewPresetTooltip,
					NewPresetIcon);
			});

		// Present the Dialog
		SGenericDialogWidget::OpenDialog(LOCTEXT("ToolPresets_CreatePreset", "Create new preset from active tool's settings"),
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ToolPresets_CreatePresetLabel", "Label"))
				]
				+ SHorizontalBox::Slot()
				.MaxWidth(300)
				[
					SNew(SEditableTextBox)
					// Cap the number of characters sent out of the text box, so we don't overflow menus and tooltips
					.OnTextCommitted_Lambda([this](const FText& NewLabel, const ETextCommit::Type&) { NewPresetLabel = NewLabel.ToString().Left(255); })
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					.ToolTipText(LOCTEXT("ToolPresets_CreatePresetLabel_Tooltip", "A short, descriptive identifier for the new preset."))
				]
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ToolPresets_CreatePresetTooltip", "Tooltip"))
				]
				+ SHorizontalBox::Slot()
				.MaxWidth(300)
				[
					SNew(SBox)
					.MinDesiredHeight(44.f)
					.MaxDesiredHeight(44.0f)
					[
						SNew(SMultiLineEditableTextBox)
						// Cap the number of characters sent out of the text box, so we don't overflow menus and tooltips
						.OnTextCommitted_Lambda([this](const FText& NewToolTip, const ETextCommit::Type&) { NewPresetTooltip = NewToolTip.ToString().Left(2048); })
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						.AllowMultiLine(false)
						.AutoWrapText(true)
						.WrappingPolicy(ETextWrappingPolicy::DefaultWrapping)
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						.ToolTipText(LOCTEXT("ToolPresets_CreatePresetTooltip_Tooltip", "A descriptive tooltip for the new preset."))
					]
				]
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SToolInputAssetComboPanel)
					.AssetClassType(UInteractiveToolsPresetCollectionAsset::StaticClass())
					.OnSelectionChanged(this, &FModelingToolsEditorModeToolkit::HandlePresetAssetChanged)
					.ToolTipText(LOCTEXT("ToolPresets_CreatePresetCollection_Tooltip", "The asset in which to store this new preset."))
					//.RecentAssetsProvider(RecentPresetCollectionProvider) // TODO: Improve this widget before enabling this feature
					.InitiallySelectedAsset(*CurrentPreset)
					.FlyoutTileSize(FVector2D(80, 80))
					.ComboButtonTileSize(FVector2D(80, 80))
					.AssetThumbnailLabel(EThumbnailLabel::AssetName)
					.bForceShowPluginContent(true)
					.bForceShowEngineContent(true)
					.AssetViewType(EAssetViewType::List)
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(10, 5)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ToolPresets_CreatePresetCollection", "Collection"))
						.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(),12, "Bold"))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(10, 5)
					[
						SNew(STextBlock)
						.Text_Lambda([this](){
						if (CurrentPresetLabel.IsEmpty())
						{
							return LOCTEXT("NewPresetNoCollectionSpecifiedMessage", "None - Preset will be added to the default Personal Presets Collection.");
						}
						else {
							return CurrentPresetLabel;
						}})
					]

				]					
			],
			FolderDialogArguments, true);
	};

	constexpr bool bShouldCloseMenuAfterSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseMenuAfterSelection, nullptr);

	constexpr bool bNoIndent = false;
	constexpr bool bSearchable = false;
	static const FName NoExtensionHook = NAME_None;


	{
		FMenuEntryParams MenuEntryParams;

		typedef TMap<FString, TArray<TSharedPtr<FToolPresetOption>>> FPresetsByNameMap;
		FPresetsByNameMap PresetsByCollectionName;
		for (TSharedPtr<FToolPresetOption> ToolPresetOption : AvailablePresetsForTool)
		{
			FModelingToolsEditorModeToolkitLocals::ExecuteWithPreset(ToolPresetOption->PresetCollection,
				[this, &PresetsByCollectionName, &ToolPresetOption](UInteractiveToolsPresetCollectionAsset& Preset) {
					PresetsByCollectionName.FindOrAdd(Preset.CollectionLabel.ToString()).Add(ToolPresetOption);
				});
		}

		for (FPresetsByNameMap::TConstIterator Iterator = PresetsByCollectionName.CreateConstIterator(); Iterator; ++Iterator)
		{
			MenuBuilder.BeginSection(NoExtensionHook, FText::FromString(Iterator.Key()));
			for (const TSharedPtr<FToolPresetOption>& ToolPresetOption : Iterator.Value())
			{
				FUIAction ApplyPresetAction;
				ApplyPresetAction.ExecuteAction = FExecuteAction::CreateLambda([this, ToolPresetOption]()
					{
						LoadPresetFromCollection(ToolPresetOption->PresetIndex, ToolPresetOption->PresetCollection);
					});

				ApplyPresetAction.CanExecuteAction = FCanExecuteAction::CreateLambda([this, ToolPresetOption]()
					{
						return this->OwningEditorMode->GetToolManager()->GetActiveTool(EToolSide::Left) != nullptr;
					});

				MenuBuilder.AddMenuEntry(
					FText::FromString(ToolPresetOption->PresetLabel),
					FText::FromString(ToolPresetOption->PresetTooltip),
					ToolPresetOption->PresetIcon,
					ApplyPresetAction);
			}
			MenuBuilder.EndSection();
		}

		MenuBuilder.BeginSection(NoExtensionHook, LOCTEXT("ModelingPresetPanelHeaderManagePresets", "Manage Presets"));


		FUIAction CreateNewPresetAction;
		CreateNewPresetAction.ExecuteAction = FExecuteAction::CreateLambda(OpenNewPresetDialog);
		MenuBuilder.AddMenuEntry(
			FText(LOCTEXT("ModelingPresetPanelCreateNewPreset", "Create New Preset")),
			FText(LOCTEXT("ModelingPresetPanelCreateNewPresetTooltip", "Create New Preset in specified Collection")),
			FSlateIcon( FAppStyle::Get().GetStyleSetName(), "Icons.Plus"),
			CreateNewPresetAction);


		FUIAction OpenPresetManangerAction;
		OpenPresetManangerAction.ExecuteAction = FExecuteAction::CreateLambda([this]() {
			IToolPresetEditorModule::Get().ExecuteOpenPresetEditor();
		});
		MenuBuilder.AddMenuEntry(
			FText(LOCTEXT("ModelingPresetPanelOpenPresetMananger", "Manage Presets...")),
			FText(LOCTEXT("ModelingPresetPanelOpenPresetManagerTooltip", "Open Preset Manager to manage presets and their collections")),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Settings"),
			OpenPresetManangerAction);

		MenuBuilder.EndSection();

	}

	return MenuBuilder.MakeWidget();
}

void FModelingToolsEditorModeToolkit::ClearPresetComboList()
{
	AvailablePresetsForTool.Empty();
}

void FModelingToolsEditorModeToolkit::RebuildPresetListForTool(bool bSettingsOpened)
{	
	TObjectPtr<UToolPresetUserSettings> UserSettings = UToolPresetUserSettings::Get();
	UserSettings->LoadEditorConfig();

	// We need to generate a combined list of Project Loaded and User available presets to intersect the enabled set against...
	const UToolPresetProjectSettings* ProjectSettings = GetDefault<UToolPresetProjectSettings>();
	TSet<FSoftObjectPath> AllUserPresets;
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	
	TArray<FAssetData> AssetData;
	FARFilter Filter;
	Filter.ClassPaths.Add(UInteractiveToolsPresetCollectionAsset::StaticClass()->GetClassPathName());
	Filter.PackagePaths.Add(FName("/ToolPresets"));
	Filter.bRecursiveClasses = false;
	Filter.bRecursivePaths = true;
	Filter.bIncludeOnlyOnDiskAssets = false;
	
	AssetRegistryModule.Get().GetAssets(Filter, AssetData);
	for (int i = 0; i < AssetData.Num(); i++) {
		UInteractiveToolsPresetCollectionAsset* Object = Cast<UInteractiveToolsPresetCollectionAsset>(AssetData[i].GetAsset());
		if (Object)
		{
			AllUserPresets.Add(Object->GetPathName());
		}
	}

	TSet<FSoftObjectPath> AvailablePresetCollections = UserSettings->EnabledPresetCollections.Intersect( ProjectSettings->LoadedPresetCollections.Union(AllUserPresets));
	if (UserSettings->bDefaultCollectionEnabled)
	{
		AvailablePresetCollections.Add(FSoftObjectPath());
	}


	AvailablePresetsForTool.Empty();
	for (const FSoftObjectPath& PresetCollection : AvailablePresetCollections)
	{
		FModelingToolsEditorModeToolkitLocals::ExecuteWithPresetAndTool(*OwningEditorMode, EToolSide::Left, PresetCollection,
			[this, PresetCollection](UInteractiveToolsPresetCollectionAsset& Preset, UInteractiveTool& Tool) {
		
				if (!Preset.PerToolPresets.Contains(Tool.GetClass()->GetName()))
				{
					return;
				}
				AvailablePresetsForTool.Reserve(Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets.Num());
				for (int32 PresetIndex = 0; PresetIndex < Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets.Num(); ++PresetIndex)
				{
					const FInteractiveToolPresetDefinition& PresetDef = Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets[PresetIndex];
					if (!PresetDef.IsValid())
					{
						continue;
					}
					TSharedPtr<FToolPresetOption> NewOption = MakeShared<FToolPresetOption>();
					if (PresetDef.Label.Len() > 50)
					{
						NewOption->PresetLabel = PresetDef.Label.Left(50) + FString("...");
					}
					else
					{
						NewOption->PresetLabel = PresetDef.Label;
					}
					if (PresetDef.Tooltip.Len() > 2048)
					{
						NewOption->PresetTooltip = PresetDef.Tooltip.Left(2048) + FString("...");
					}
					else
					{
						NewOption->PresetTooltip = PresetDef.Tooltip;
					}
					NewOption->PresetIndex = PresetIndex;
					NewOption->PresetCollection = PresetCollection;

					AvailablePresetsForTool.Add(NewOption);
				}
			});
	}
}

void FModelingToolsEditorModeToolkit::HandlePresetAssetChanged(const FAssetData& InAssetData)
{
	CurrentPresetPath.SetPath(InAssetData.GetObjectPathString());
	CurrentPresetLabel = FText();
	*CurrentPreset = InAssetData;

	UInteractiveToolsPresetCollectionAsset* Preset = nullptr;
	if (CurrentPresetPath.IsAsset())
	{
		Preset = Cast<UInteractiveToolsPresetCollectionAsset>(CurrentPresetPath.TryLoad());
	}
	if (Preset)
	{
		CurrentPresetLabel = Preset->CollectionLabel;
	}
	
}

bool FModelingToolsEditorModeToolkit::HandleFilterPresetAsset(const FAssetData& InAssetData)
{
	return false;
}

void FModelingToolsEditorModeToolkit::LoadPresetFromCollection(const int32 PresetIndex, FSoftObjectPath CollectionPath)
{
	FModelingToolsEditorModeToolkitLocals::ExecuteWithPresetAndTool(*OwningEditorMode, EToolSide::Left, CollectionPath,
	[this, PresetIndex](UInteractiveToolsPresetCollectionAsset& Preset, UInteractiveTool& Tool) {
		TArray<UObject*> PropertySets = Tool.GetToolProperties();

		// We only want to load the properties that are actual property sets, since the tool might have added other types of objects we don't
		// want to deserialize.
		PropertySets.RemoveAll([this](UObject* Object) {
			return Cast<UInteractiveToolPropertySet>(Object) == nullptr;
		});

		if (!Preset.PerToolPresets.Contains(Tool.GetClass()->GetName()))			
		{
			return;
		}
		if(PresetIndex < 0 || PresetIndex >= Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets.Num())
		{
			return;
		}

		Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets[PresetIndex].LoadStoredPropertyData(PropertySets);

	});
}

void FModelingToolsEditorModeToolkit::CreateNewPresetInCollection(const FString& PresetLabel, FSoftObjectPath CollectionPath, const FString& ToolTip, FSlateIcon Icon)
{
	FModelingToolsEditorModeToolkitLocals::ExecuteWithPresetAndTool(*OwningEditorMode, EToolSide::Left, CurrentPresetPath,
	[this, PresetLabel, ToolTip, Icon, CollectionPath](UInteractiveToolsPresetCollectionAsset& Preset, UInteractiveTool& Tool) {

		TArray<UObject*> PropertySets = Tool.GetToolProperties();

		// We only want to add the properties that are actual property sets, since the tool might have added other types of objects we don't
		// want to serialize.
		PropertySets.RemoveAll([this](UObject* Object) {
			return Cast<UInteractiveToolPropertySet>(Object) == nullptr;
		});

		Preset.PerToolPresets.FindOrAdd(Tool.GetClass()->GetName()).ToolLabel = ActiveToolName;
		if (ensure(ActiveToolIcon))
		{
			Preset.PerToolPresets.FindOrAdd(Tool.GetClass()->GetName()).ToolIcon = *ActiveToolIcon;
		}
		FInteractiveToolPresetDefinition PresetValuesToCreate;
		if (PresetLabel.IsEmpty())
		{
			PresetValuesToCreate.Label = FString::Printf(TEXT("Unnamed_Preset-%d"), Preset.PerToolPresets.FindOrAdd(Tool.GetClass()->GetName()).NamedPresets.Num()+1);
		}
		else
		{
			PresetValuesToCreate.Label = PresetLabel;
		}
		PresetValuesToCreate.Tooltip = ToolTip;
		//PresetValuesToCreate.Icon = Icon;

		PresetValuesToCreate.SetStoredPropertyData(PropertySets);


		Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets.Add(PresetValuesToCreate);
		Preset.MarkPackageDirty();

		// Finally add this to the current tool's preset list, since we know it should be there.
		TSharedPtr<FToolPresetOption> NewOption = MakeShared<FToolPresetOption>();
		NewOption->PresetLabel = PresetLabel;
		NewOption->PresetTooltip = ToolTip;
		NewOption->PresetIndex = Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets.Num() - 1;
		NewOption->PresetCollection = CollectionPath;

		AvailablePresetsForTool.Add(NewOption);
		
	});

	UToolPresetAssetSubsystem* PresetAssetSubsystem = GEditor->GetEditorSubsystem<UToolPresetAssetSubsystem>();
	if (CollectionPath.IsNull() && ensure(PresetAssetSubsystem))
	{
		ensure(PresetAssetSubsystem->SaveDefaultCollection());
	}
}

void FModelingToolsEditorModeToolkit::UpdatePresetInCollection(const FToolPresetOption& PresetToEditIn, bool bUpdateStoredPresetValues)
{
	FModelingToolsEditorModeToolkitLocals::ExecuteWithPresetAndTool(*OwningEditorMode, EToolSide::Left, PresetToEditIn.PresetCollection,
		[this, PresetToEditIn, bUpdateStoredPresetValues](UInteractiveToolsPresetCollectionAsset& Preset, UInteractiveTool& Tool) {
			TArray<UObject*> PropertySets = Tool.GetToolProperties();

			// We only want to add the properties that are actual property sets, since the tool might have added other types of objects we don't
			// want to serialize.
			PropertySets.RemoveAll([this](UObject* Object) {
				return Cast<UInteractiveToolPropertySet>(Object) == nullptr;
			});

			if (!Preset.PerToolPresets.Contains(Tool.GetClass()->GetName()))
			{
				return;
			}
			if (PresetToEditIn.PresetIndex < 0 || PresetToEditIn.PresetIndex >= Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets.Num() )
			{
				return;
			}

			if (bUpdateStoredPresetValues)
			{
				Preset.PerToolPresets.FindOrAdd(Tool.GetClass()->GetName()).NamedPresets[PresetToEditIn.PresetIndex].SetStoredPropertyData(PropertySets);
				Preset.MarkPackageDirty();
			}

			Preset.PerToolPresets.FindOrAdd(Tool.GetClass()->GetName()).NamedPresets[PresetToEditIn.PresetIndex].Label = PresetToEditIn.PresetLabel;
			Preset.PerToolPresets.FindOrAdd(Tool.GetClass()->GetName()).NamedPresets[PresetToEditIn.PresetIndex].Tooltip = PresetToEditIn.PresetTooltip;

		});

	RebuildPresetListForTool(false);

	UToolPresetAssetSubsystem* PresetAssetSubsystem = GEditor->GetEditorSubsystem<UToolPresetAssetSubsystem>();
	if (PresetToEditIn.PresetCollection.IsNull() && ensure(PresetAssetSubsystem))
	{
		ensure(PresetAssetSubsystem->SaveDefaultCollection());
	}
}




void FModelingToolsEditorModeToolkit::InitializeAfterModeSetup()
{
	if (bFirstInitializeAfterModeSetup)
	{
		// force update of the active asset LOD mode, this is necessary because the update modifies
		// ToolTarget Factories that are only available once ModelingToolsEditorMode has been initialized
		AssetLODMode->SetSelectedItem(AssetLODModes[0]);

		bFirstInitializeAfterModeSetup = false;
	}

}



void FModelingToolsEditorModeToolkit::UpdateActiveToolProperties()
{
	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
	if (CurTool == nullptr)
	{
		return;
	}

	// Before actually changing the detail panel, we need to see where the current keyboard focus is, because
	// if it's inside the detail panel, we'll need to reset it to the detail panel as a whole, else we might
	// lose it entirely when that detail panel element gets destroyed (which would make us unable to receive any
	// hotkey presses until the user clicks somewhere).
	TSharedPtr<SWidget> FocusedWidget = FSlateApplication::Get().GetKeyboardFocusedWidget();
	if (FocusedWidget != ModeDetailsView) 
	{
		// Search upward from the currently focused widget
		TSharedPtr<SWidget> CurrentWidget = FocusedWidget;
		while (CurrentWidget.IsValid())
		{
			if (CurrentWidget == ModeDetailsView)
			{
				// Reset focus to the detail panel as a whole to avoid losing it when the inner elements change.
				FSlateApplication::Get().SetKeyboardFocus(ModeDetailsView);
				break;
			}

			CurrentWidget = CurrentWidget->GetParentWidget();
		}
	}
		
	ModeDetailsView->SetObjects(CurTool->GetToolProperties(true));
}

void FModelingToolsEditorModeToolkit::InvalidateCachedDetailPanelState(UObject* ChangedObject)
{
	ModeDetailsView->InvalidateCachedState();
}


void FModelingToolsEditorModeToolkit::PostNotification(const FText& Message)
{
	ClearNotification();

	ActiveToolMessage = Message;

	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		ActiveToolMessageHandle = GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PushStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), ActiveToolMessage);
	}
}

void FModelingToolsEditorModeToolkit::ClearNotification()
{
	ActiveToolMessage = FText::GetEmpty();

	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PopStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), ActiveToolMessageHandle);
	}
	ActiveToolMessageHandle.Reset();
}


void FModelingToolsEditorModeToolkit::PostWarning(const FText& Message)
{
	ToolWarningArea->SetText(Message);
	ToolWarningArea->SetVisibility(EVisibility::Visible);
}

void FModelingToolsEditorModeToolkit::ClearWarning()
{
	ToolWarningArea->SetText(FText());
	ToolWarningArea->SetVisibility(EVisibility::Collapsed);
}

FName FModelingToolsEditorModeToolkit::GetToolkitFName() const
{
	return FName("ModelingToolsEditorMode");
}

FText FModelingToolsEditorModeToolkit::GetBaseToolkitName() const
{
	return NSLOCTEXT("ModelingToolsEditorModeToolkit", "DisplayName", "ModelingToolsEditorMode Tool");
}

static const FName PrimitiveTabName(TEXT("Shapes"));
static const FName CreateTabName(TEXT("Create"));
static const FName AttributesTabName(TEXT("Attributes"));
static const FName TriModelingTabName(TEXT("TriModel"));
static const FName PolyModelingTabName(TEXT("PolyModel"));
static const FName MeshProcessingTabName(TEXT("MeshOps"));
static const FName UVTabName(TEXT("UVs"));
static const FName TransformTabName(TEXT("Transform"));
static const FName DeformTabName(TEXT("Deform"));
static const FName VolumesTabName(TEXT("Volumes"));
static const FName PrototypesTabName(TEXT("Prototypes"));
static const FName PolyEditTabName(TEXT("PolyEdit"));
static const FName VoxToolsTabName(TEXT("VoxOps"));
static const FName LODToolsTabName(TEXT("LODs"));
static const FName BakingToolsTabName(TEXT("Baking"));
static const FName ModelingFavoritesTabName(TEXT("Favorites"));

static const FName SelectionActionsTabName(TEXT("Selection"));


const TArray<FName> FModelingToolsEditorModeToolkit::PaletteNames_Standard = { PrimitiveTabName, CreateTabName, PolyModelingTabName, TriModelingTabName, DeformTabName, TransformTabName, MeshProcessingTabName, VoxToolsTabName, AttributesTabName, UVTabName, BakingToolsTabName, VolumesTabName, LODToolsTabName };


void FModelingToolsEditorModeToolkit::GetToolPaletteNames(TArray<FName>& PaletteNames) const
{
	PaletteNames = PaletteNames_Standard;

	TArray<FName> ExistingNames;
	for ( FName Name : PaletteNames )
	{
		ExistingNames.Add(Name);
	}

	const UModelingToolsEditorModeSettings* ModelingModeSettings = GetDefault<UModelingToolsEditorModeSettings>();
	bool bEnableSelectionUI = ModelingModeSettings && ModelingModeSettings->GetMeshSelectionsEnabled();
	if (bEnableSelectionUI)
	{
		if (bShowActiveSelectionActions)
		{
			PaletteNames.Insert(SelectionActionsTabName, 0);
			ExistingNames.Add(SelectionActionsTabName);
		}
	}

	bool bEnablePrototypes = (CVarEnablePrototypeModelingTools.GetValueOnGameThread() > 0);
	if (bEnablePrototypes)
	{
		PaletteNames.Add(PrototypesTabName);
		ExistingNames.Add(PrototypesTabName);
	}

	bool bEnablePolyModel = (CVarEnablePolyModeling.GetValueOnGameThread() > 0);
	if (bEnablePolyModel)
	{
		PaletteNames.Add(PolyEditTabName);
		ExistingNames.Add(PolyEditTabName);
	}

	if (IModularFeatures::Get().IsModularFeatureAvailable(IModelingModeToolExtension::GetModularFeatureName()))
	{
		TArray<IModelingModeToolExtension*> Extensions = IModularFeatures::Get().GetModularFeatureImplementations<IModelingModeToolExtension>(
			IModelingModeToolExtension::GetModularFeatureName());
		for (int32 k = 0; k < Extensions.Num(); ++k)
		{
			FText ExtensionName = Extensions[k]->GetExtensionName();
			FText SectionName = Extensions[k]->GetToolSectionName();
			FName SectionIndex(SectionName.ToString());
			if (ExistingNames.Contains(SectionIndex))
			{
				UE_LOGF(LogTemp, Warning, "Modeling Mode Extension [%ls] uses existing Section Name [%ls] - buttons may not be visible", *ExtensionName.ToString(), *SectionName.ToString());
			}
			else
			{
				PaletteNames.Add(SectionIndex);
				ExistingNames.Add(SectionIndex);
			}
		}
	}

	UModelingToolsModeCustomizationSettings* UISettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>();

	// if user has provided custom ordering of tool palettes in the Editor Settings, try to apply them
	if (UISettings->ToolSectionOrder.Num() > 0)
	{
		TArray<FName> NewPaletteNames;
		for (FString SectionName : UISettings->ToolSectionOrder)
		{
			for (int32 k = 0; k < PaletteNames.Num(); ++k)
			{
				if (SectionName.Equals(PaletteNames[k].ToString(), ESearchCase::IgnoreCase)
				 || SectionName.Equals(GetToolPaletteDisplayName(PaletteNames[k]).ToString(), ESearchCase::IgnoreCase))
				{
					NewPaletteNames.Add(PaletteNames[k]);
					PaletteNames.RemoveAt(k);
					break;
				}
			}
		}
		NewPaletteNames.Append(PaletteNames);
		PaletteNames = MoveTemp(NewPaletteNames);
	}

	// if user has provided a list of favorite tools, add that palette to the list
	if (FavoritesPalette && FavoritesPalette->GetPaletteCommandNames().Num() > 0)
	{
		PaletteNames.Insert(ModelingFavoritesTabName, 0);
	}

}


FText FModelingToolsEditorModeToolkit::GetToolPaletteDisplayName(FName Palette) const
{
	return FText::FromName(Palette);
}

void FModelingToolsEditorModeToolkit::BuildToolPalette(FName PaletteIndex, class FToolBarBuilder& ToolbarBuilder) 
{

	if (HasToolkitBuilder())
	{
		return;
	}
	
	const FModelingToolsManagerCommands& Commands = FModelingToolsManagerCommands::Get();
	UModelingToolsModeCustomizationSettings* UISettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>();
	
	bool bUseShortNames = UISettings->bUseLegacyModelingPalette;
	auto AddButton = [&Commands, bUseShortNames, &ToolbarBuilder](const TSharedPtr<const FUICommandInfo>& Command)
	{
		ToolbarBuilder.AddToolBarButton(Command, NAME_None,
			Commands.GetCommandLabel(Command, bUseShortNames));
	};

	if (PaletteIndex == ModelingFavoritesTabName)
	{
		// build Favorites tool palette
		TArray<TSharedPtr<const FUICommandInfo>> FavoriteCommands;
		ToolkitBuilder->GetCommandsForEditablePalette(FavoritesPalette.ToSharedRef(), FavoriteCommands);
		
		for (const TSharedPtr<const FUICommandInfo>& ToolCommand : FavoriteCommands)
		{
			AddButton(ToolCommand);
		}
	}
	else if (PaletteIndex == SelectionActionsTabName)
	{
		AddButton(Commands.BeginSelectionAction_Delete);
		//AddButton(Commands.BeginSelectionAction_Disconnect);		// disabled for 5.2, available via TriSel Tool
		AddButton(Commands.BeginSelectionAction_Extrude);
		AddButton(Commands.BeginSelectionAction_Offset);

		AddButton(Commands.BeginPolyModelTool_ExtrudeEdges);
		AddButton(Commands.BeginPolyModelTool_PushPull);
		AddButton(Commands.BeginPolyModelTool_Inset);
		AddButton(Commands.BeginPolyModelTool_Outset);
		AddButton(Commands.BeginPolyModelTool_CutFaces);
		AddButton(Commands.BeginPolyModelTool_Bevel);

		AddButton(Commands.BeginPolyModelTool_InsertEdgeLoop);
		AddButton(Commands.BeginSelectionAction_Retriangulate);

		AddButton(Commands.BeginPolyModelTool_PolyEd);
		AddButton(Commands.BeginPolyModelTool_TriSel);
		
	}
	else if (PaletteIndex == PrimitiveTabName)
	{
		AddButton(Commands.BeginAddBoxPrimitiveTool);
		AddButton(Commands.BeginAddSpherePrimitiveTool);
		AddButton(Commands.BeginAddCylinderPrimitiveTool);
		AddButton(Commands.BeginAddCapsulePrimitiveTool);
		AddButton(Commands.BeginAddConePrimitiveTool);
		AddButton(Commands.BeginAddTorusPrimitiveTool);
		AddButton(Commands.BeginAddArrowPrimitiveTool);
		AddButton(Commands.BeginAddRectanglePrimitiveTool);
		AddButton(Commands.BeginAddDiscPrimitiveTool);
		AddButton(Commands.BeginAddStairsPrimitiveTool);
	}
	else if (PaletteIndex == CreateTabName)
	{
		AddButton(Commands.BeginDrawPolygonTool);
		AddButton(Commands.BeginDrawPolyPathTool);
		AddButton(Commands.BeginDrawAndRevolveTool);
		AddButton(Commands.BeginRevolveSplineTool);
		AddButton(Commands.BeginRevolveBoundaryTool);
		AddButton(Commands.BeginCombineMeshesTool);
		AddButton(Commands.BeginDuplicateMeshesTool);
		AddButton(Commands.BeginPatternTool);
	}
	else if (PaletteIndex == TransformTabName)
	{
		AddButton(Commands.BeginTransformMeshesTool);
		AddButton(Commands.BeginAlignObjectsTool);
		AddButton(Commands.BeginEditPivotTool);
		AddButton(Commands.BeginAddPivotActorTool);
		AddButton(Commands.BeginBakeTransformTool);
		AddButton(Commands.BeginTransferMeshTool);
		AddButton(Commands.BeginConvertMeshesTool);
		AddButton(Commands.BeginSplitMeshesTool);
	}
	else if (PaletteIndex == DeformTabName)
	{
		AddButton(Commands.BeginSculptMeshTool);
		AddButton(Commands.BeginRemeshSculptMeshTool);
		AddButton(Commands.BeginSmoothMeshTool);
		AddButton(Commands.BeginOffsetMeshTool);
		AddButton(Commands.BeginMeshSpaceDeformerTool);
		AddButton(Commands.BeginLatticeDeformerTool);
		AddButton(Commands.BeginDisplaceMeshTool);
	}
	else if (PaletteIndex == MeshProcessingTabName)
	{
		AddButton(Commands.BeginSimplifyMeshTool);
		AddButton(Commands.BeginRemeshMeshTool);
		AddButton(Commands.BeginWeldEdgesTool);
		AddButton(Commands.BeginRemoveOccludedTrianglesTool);
		AddButton(Commands.BeginSelfUnionTool);
		AddButton(Commands.BeginProjectToTargetTool);
	}
	else if (PaletteIndex == LODToolsTabName)
	{
		AddButton(Commands.BeginLODManagerTool);
		AddButton(Commands.BeginGenerateStaticMeshLODAssetTool);
		AddButton(Commands.BeginISMEditorTool);
	}
	else if (PaletteIndex == VoxToolsTabName)
	{
		AddButton(Commands.BeginVoxelSolidifyTool);
		AddButton(Commands.BeginVoxelBlendTool);
		AddButton(Commands.BeginVoxelMorphologyTool);
#if WITH_PROXYLOD
		AddButton(Commands.BeginVoxelBooleanTool);
		AddButton(Commands.BeginVoxelMergeTool);
#endif	// WITH_PROXYLOD
	}
	else if (PaletteIndex == TriModelingTabName)
	{
		AddButton(Commands.BeginMeshSelectionTool);
		AddButton(Commands.BeginTriEditTool);
		AddButton(Commands.BeginHoleFillTool);
		AddButton(Commands.BeginMirrorTool);
		AddButton(Commands.BeginPlaneCutTool);
		AddButton(Commands.BeginPolygonCutTool);
		AddButton(Commands.BeginMeshTrimTool);
	}
	else if (PaletteIndex == PolyModelingTabName)
	{
		AddButton(Commands.BeginPolyEditTool);
		AddButton(Commands.BeginPolyDeformTool);
		AddButton(Commands.BeginCubeGridTool);
		AddButton(Commands.BeginMeshBooleanTool);
		AddButton(Commands.BeginCutMeshWithMeshTool);
		AddButton(Commands.BeginSubdividePolyTool);
	}
	else if (PaletteIndex == AttributesTabName)
	{
		AddButton(Commands.BeginMeshInspectorTool);
		AddButton(Commands.BeginEditNormalsTool);
		AddButton(Commands.BeginEditTangentsTool);
		AddButton(Commands.BeginAttributeEditorTool);
		AddButton(Commands.BeginPolyGroupsTool);
		AddButton(Commands.BeginMeshGroupPaintTool);
		AddButton(Commands.BeginMeshAttributePaintTool);
		AddButton(Commands.BeginEditMeshMaterialsTool);
	} 
	else if (PaletteIndex == BakingToolsTabName )
	{
		AddButton(Commands.BeginBakeMeshAttributeMapsTool);
		AddButton(Commands.BeginBakeMultiMeshAttributeMapsTool);
		AddButton(Commands.BeginBakeMeshAttributeVertexTool);
		AddButton(Commands.BeginBakeRenderCaptureTool);
	}
	else if (PaletteIndex == UVTabName)
	{
		AddButton(Commands.BeginGlobalUVGenerateTool);
		AddButton(Commands.BeginGroupUVGenerateTool);
		AddButton(Commands.BeginUVProjectionTool);
		AddButton(Commands.BeginUVSeamEditTool);
		AddButton(Commands.BeginTransformUVIslandsTool);
		AddButton(Commands.BeginUVLayoutTool);
		AddButton(Commands.BeginUVTransferTool);

		// Handle the inclusion of the optional UVEditor button if the UVEditor plugin has been found
		if (IModularFeatures::Get().IsModularFeatureAvailable(IUVEditorModularFeature::GetModularFeatureName()))
		{
			AddButton(Commands.LaunchUVEditor);
		}
	}
	else if (PaletteIndex == VolumesTabName)
	{
		AddButton(Commands.BeginVolumeToMeshTool);
		AddButton(Commands.BeginMeshToVolumeTool);
		ToolbarBuilder.AddSeparator();

		// BSPConv is disabled in Restrictive Mode.
		if (Commands.BeginBspConversionTool)
		{
			AddButton(Commands.BeginBspConversionTool);
			ToolbarBuilder.AddSeparator();
		}

		AddButton(Commands.BeginPhysicsInspectorTool);
		AddButton(Commands.BeginSetCollisionGeometryTool);
		//AddButton(Commands.BeginEditCollisionGeometryTool);
		AddButton(Commands.BeginExtractCollisionGeometryTool);
	}
	else if (PaletteIndex == PrototypesTabName)
	{
		AddButton(Commands.BeginAddPatchTool);
		AddButton(Commands.BeginShapeSprayTool);
	}
	//else if (PaletteIndex == PolyEditTabName)
	//{
	//	AddButton(Commands.BeginPolyModelTool_FaceSelect);
	//	AddButton(Commands.BeginPolyModelTool_EdgeSelect);
	//	AddButton(Commands.BeginPolyModelTool_VertexSelect);
	//	AddButton(Commands.BeginPolyModelTool_AllSelect);
	//	AddButton(Commands.BeginPolyModelTool_LoopSelect);
	//	AddButton(Commands.BeginPolyModelTool_RingSelect);
	//	ToolbarBuilder.AddSeparator();
	//	AddButton(Commands.BeginPolyModelTool_Extrude);
	//	AddButton(Commands.BeginPolyModelTool_Inset);
	//	AddButton(Commands.BeginPolyModelTool_Outset);
	//	AddButton(Commands.BeginPolyModelTool_CutFaces);
	//	ToolbarBuilder.AddSeparator();
	//	AddButton(Commands.BeginSubdividePolyTool);
	//	AddButton(Commands.BeginPolyEditTool);
	//}
	else
	{
		TArray<IModelingModeToolExtension*> Extensions = IModularFeatures::Get().GetModularFeatureImplementations<IModelingModeToolExtension>(
			IModelingModeToolExtension::GetModularFeatureName());
		for (int32 k = 0; k < Extensions.Num(); ++k)
		{
			FText SectionName = Extensions[k]->GetToolSectionName();
			FName SectionIndex(SectionName.ToString());
			if (PaletteIndex == SectionIndex)
			{
				FExtensionToolQueryInfo ExtensionQueryInfo;
				ExtensionQueryInfo.bIsInfoQueryOnly = true;
				TArray<FExtensionToolDescription> ToolSet;
				Extensions[k]->GetExtensionTools(ExtensionQueryInfo, ToolSet);
				for (const FExtensionToolDescription& ToolInfo : ToolSet)
				{
					AddButton(ToolInfo.ToolCommand);
				}
			}
		}
	}
}


void FModelingToolsEditorModeToolkit::InvokeUI()
{
	FModeToolkit::InvokeUI();

	// FModeToolkit::UpdatePrimaryModePanel() wrapped our GetInlineContent() output in a SScrollBar widget,
	// however this doesn't make sense as we want to dock panels to the "top" and "bottom" of our mode panel area,
	// and the details panel in the middle has it's own scrollbar already. The SScrollBar is hardcoded as the content
	// of FModeToolkit::InlineContentHolder so we can just replace it here
	InlineContentHolder->SetContent(GetInlineContent().ToSharedRef());

	// if the ToolkitBuilder is being used, it will make up the UI
	if (HasToolkitBuilder())
	{
		return;
	}

	//
	// Apply custom section header colors.
	// See comments below, this is done via directly manipulating Slate widgets generated deep inside BaseToolkit.cpp,
	// and will stop working if the Slate widget structure changes
	//

	UModelingToolsModeCustomizationSettings* UISettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>();

	// Generate a map for tool specific colors
	TMap<FString, FLinearColor> SectionIconColorMap;
	TMap<FString, TMap<FString, FLinearColor>> SectionToolIconColorMap;
	for (const FModelingModeCustomToolColor& ToolColor : UISettings->ToolColors)
	{
		FString SectionName, ToolName;
		ToolColor.ToolName.Split(".", &SectionName, &ToolName);
		SectionName.ToLowerInline();
		if (ToolName.Len() > 0)
		{
			if (!SectionToolIconColorMap.Contains(SectionName))
			{
				SectionToolIconColorMap.Emplace(SectionName, TMap<FString, FLinearColor>());
			}
			SectionToolIconColorMap[SectionName].Add(ToolName, ToolColor.Color);
		}
		else
		{
			SectionIconColorMap.Emplace(ToolColor.ToolName.ToLower(), ToolColor.Color);
		}
	}

	for (FEdModeToolbarRow& ToolbarRow : ActiveToolBarRows)
	{
		// Update section header colors
		for (FModelingModeCustomSectionColor ToolColor : UISettings->SectionColors)
		{
			if (ToolColor.SectionName.Equals(ToolbarRow.PaletteName.ToString(), ESearchCase::IgnoreCase)
			 || ToolColor.SectionName.Equals(ToolbarRow.DisplayName.ToString(), ESearchCase::IgnoreCase))
			{
				// code below is highly dependent on the structure of the ToolbarRow.ToolbarWidget. Currently this is 
				// a SMultiBoxWidget, a few levels below a SExpandableArea. The SExpandableArea contains a SVerticalBox
				// with the header as a SBorder in Slot 0. The code will fail gracefully if this structure changes.

				TSharedPtr<SWidget> ExpanderVBoxWidget = (ToolbarRow.ToolbarWidget.IsValid() && ToolbarRow.ToolbarWidget->GetParentWidget().IsValid()) ?
					ToolbarRow.ToolbarWidget->GetParentWidget()->GetParentWidget() : TSharedPtr<SWidget>();
				if (ExpanderVBoxWidget.IsValid() && ExpanderVBoxWidget->GetTypeAsString().Compare(TEXT("SVerticalBox")) == 0)
				{
					TSharedPtr<SVerticalBox> ExpanderVBox = StaticCastSharedPtr<SVerticalBox>(ExpanderVBoxWidget);
					if (ExpanderVBox.IsValid() && ExpanderVBox->NumSlots() > 0)
					{
						const TSharedRef<SWidget>& SlotWidgetRef = ExpanderVBox->GetSlot(0).GetWidget();
						TSharedPtr<SWidget> SlotWidgetPtr(SlotWidgetRef);
						if (SlotWidgetPtr.IsValid() && SlotWidgetPtr->GetTypeAsString().Compare(TEXT("SBorder")) == 0)
						{
							TSharedPtr<SBorder> TopBorder = StaticCastSharedPtr<SBorder>(SlotWidgetPtr);
							if (TopBorder.IsValid())
							{
								// The border image needs to be swapped to a white one so that the color applied to it is
								// not darkened by the usual brush. Note that we can't just create a new FSlateRoundedBoxBrush
								// with the proper color in-line because a brush needs to be associated with a style set that
								// is responsible for freeing it, else it will leak (or we could own the brush in the toolkit,
								// but this is frowned upon).
								TopBorder->SetBorderImage(FModelingToolsEditorModeStyle::Get()->GetBrush("ModelingMode.WhiteExpandableAreaHeader"));
								TopBorder->SetBorderBackgroundColor(FSlateColor(ToolColor.Color));
							}
						}
					}
				}
				break;
			}
		}

		// Update tool colors
		FLinearColor* SectionIconColor = SectionIconColorMap.Find(ToolbarRow.PaletteName.ToString().ToLower());
		if (!SectionIconColor)
		{
			SectionIconColor = SectionIconColorMap.Find(ToolbarRow.DisplayName.ToString().ToLower());
		}
		TMap<FString, FLinearColor>* SectionToolIconColors = SectionToolIconColorMap.Find(ToolbarRow.PaletteName.ToString().ToLower());
		if (!SectionToolIconColors)
		{
			SectionToolIconColors = SectionToolIconColorMap.Find(ToolbarRow.DisplayName.ToString().ToLower());
		}
		if (SectionIconColor || SectionToolIconColors)
		{
			// code below is highly dependent on the structure of the ToolbarRow.ToolbarWidget. Currently this is 
			// a SMultiBoxWidget. The code will fail gracefully if this structure changes.
			
			if (ToolbarRow.ToolbarWidget.IsValid() && ToolbarRow.ToolbarWidget->GetTypeAsString().Compare(TEXT("SMultiBoxWidget")) == 0)
			{
				auto FindFirstChildWidget = [](const TSharedPtr<SWidget>& Widget, const FString& WidgetType)
				{
					TSharedPtr<SWidget> Result;
					UE::ModelingUI::ProcessChildWidgetsByType(Widget->AsShared(), WidgetType,[&Result](TSharedRef<SWidget> Widget)
					{
						Result = TSharedPtr<SWidget>(Widget);
						// Stop processing after first occurrence
						return false;
					});
					return Result;
				};

				TSharedPtr<SWidget> PanelWidget = FindFirstChildWidget(ToolbarRow.ToolbarWidget, TEXT("SUniformWrapPanel"));
				if (PanelWidget.IsValid())
				{
					// This contains each of the FToolBarButtonBlock items for this row.
					FChildren* PanelChildren = PanelWidget->GetChildren();
					const int32 NumChild = PanelChildren ? PanelChildren->NumSlot() : 0;
					for (int32 ChildIdx = 0; ChildIdx < NumChild; ++ChildIdx)
					{
						const TSharedRef<SWidget> ChildWidgetRef = PanelChildren->GetChildAt(ChildIdx);
						TSharedPtr<SWidget> ChildWidgetPtr(ChildWidgetRef);
						if (ChildWidgetPtr.IsValid() && ChildWidgetPtr->GetTypeAsString().Compare(TEXT("SToolBarButtonBlock")) == 0)
						{
							TSharedPtr<SToolBarButtonBlock> ToolBarButton = StaticCastSharedPtr<SToolBarButtonBlock>(ChildWidgetPtr);
							if (ToolBarButton.IsValid())
							{
								TSharedPtr<SWidget> LayeredImageWidget = FindFirstChildWidget(ToolBarButton, TEXT("SLayeredImage"));
								TSharedPtr<SWidget> TextBlockWidget = FindFirstChildWidget(ToolBarButton, TEXT("STextBlock"));
								if (LayeredImageWidget.IsValid() && TextBlockWidget.IsValid())
								{
									TSharedPtr<SImage> ImageWidget = StaticCastSharedPtr<SImage>(LayeredImageWidget);
									TSharedPtr<STextBlock> TextWidget = StaticCastSharedPtr<STextBlock>(TextBlockWidget);
									// Check if this Section.Tool has an explicit color entry. If not, fallback
									// to any Section-wide color entry, otherwise leave the tint alone.
									FLinearColor* TintColor = SectionToolIconColors ? SectionToolIconColors->Find(TextWidget->GetText().ToString()) : nullptr;
									if (!TintColor)
									{
										const FString* SourceText = FTextInspector::GetSourceString(TextWidget->GetText());
										TintColor = SectionToolIconColors && SourceText ? SectionToolIconColors->Find(*SourceText) : nullptr;
										if (!TintColor)
										{
											TintColor = SectionIconColor;
										}
									}
									if (TintColor)
									{
										ImageWidget->SetColorAndOpacity(FSlateColor(*TintColor));
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

void FModelingToolsEditorModeToolkit::ShutdownUI()
{
	FModeToolkit::ShutdownUI();

	ClearNotification();
}

void FModelingToolsEditorModeToolkit::ForceToolPaletteRebuild()
{
	// disabling this for now because it resets the expand/contract state of all the palettes...

	//UGeometrySelectionManager* SelectionManager = Cast<UModelingToolsEditorMode>(GetScriptableEditorMode())->GetSelectionManager();
	//if (SelectionManager)
	//{
	//	bool bHasActiveSelection = SelectionManager->HasSelection();
	//	if (bShowActiveSelectionActions != bHasActiveSelection)
	//	{
	//		bShowActiveSelectionActions = bHasActiveSelection;
	//		this->RebuildModeToolPalette();
	//	}
	//}
	//else
	//{
	//	bShowActiveSelectionActions = false;
	//}
}

void FModelingToolsEditorModeToolkit::BindGizmoNumericalUI()
{
	if (ensure(GizmoNumericalUIOverlayWidget.IsValid()))
	{
		ensure(GizmoNumericalUIOverlayWidget->BindToGizmoContextObject(GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)));
	}
}


UGeometrySelectionManager* FModelingToolsEditorModeToolkit::GetMeshSelectionManager()
{
	if (CachedSelectionManager == nullptr)
	{
		CachedSelectionManager = Cast<UModelingToolsEditorMode>(GetScriptableEditorMode())->GetSelectionManager();
	}
	return CachedSelectionManager;
}

void FModelingToolsEditorModeToolkit::OnToolPaletteChanged(FName PaletteName) 
{
}



void FModelingToolsEditorModeToolkit::ShowRealtimeAndModeWarnings(bool bShowRealtimeWarning)
{
	FText WarningText{};
	if (GEditor->bIsSimulatingInEditor)
	{
		WarningText = LOCTEXT("ModelingModeToolkitSimulatingWarning", "Cannot use Modeling Tools while simulating.");
	}
	else if (GEditor->PlayWorld != NULL)
	{
		WarningText = LOCTEXT("ModelingModeToolkitPIEWarning", "Cannot use Modeling Tools in PIE.");
	}
	else if (bShowRealtimeWarning)
	{
		WarningText = LOCTEXT("ModelingModeToolkitRealtimeWarning", "Realtime Mode is required for Modeling Tools to work correctly. Please enable Realtime Mode in the Viewport Options or with the Ctrl+r hotkey.");
	}
	else if (EngineAssetsSelected())
	{
		WarningText = LOCTEXT("ModelingModeToolkitEngineAssetsWarning", "Selection includes Engine assets, which cannot be modified.");
	}
	if (!WarningText.IdenticalTo(ActiveWarning))
	{
		ActiveWarning = WarningText;
		ModeWarningArea->SetVisibility(ActiveWarning.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible);
		ModeWarningArea->SetText(ActiveWarning);
	}
}

void FModelingToolsEditorModeToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	bInActiveTool = true;

	UpdateActiveToolProperties();

	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
	// This can happen due to out of order mode Enter/Exit as a result of Exit being deferred to Tick.
	// TODO: Swap this to an ensure once we fix the deferred UEdMode::Exit handling.
	if (!CurTool)
	{
		return;
	}
	
	CurTool->OnPropertySetsModified.AddSP(this, &FModelingToolsEditorModeToolkit::UpdateActiveToolProperties);
	CurTool->OnPropertyModifiedDirectlyByTool.AddSP(this, &FModelingToolsEditorModeToolkit::InvalidateCachedDetailPanelState);

	ModeHeaderArea->SetVisibility(EVisibility::Collapsed);
	ActiveToolName = CurTool->GetToolInfo().ToolDisplayName;

	if (HasToolkitBuilder())
	{
		ToolkitBuilder->SetActiveToolDisplayName(ActiveToolName);
		if (const UModelingToolsModeCustomizationSettings* Settings = GetDefault<UModelingToolsModeCustomizationSettings>())
		{
			if (Settings->bAlwaysShowToolButtons == false)
			{
				ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Collapsed);
			}
		}
	}

	// try to update icon
	FString ActiveToolIdentifier = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveToolName(EToolSide::Left);
	ActiveToolIdentifier.InsertAt(0, ".");
	FName ActiveToolIconName = ISlateStyle::Join(FModelingToolsManagerCommands::Get().GetContextName(), TCHAR_TO_ANSI(*ActiveToolIdentifier));
	ActiveToolIcon = FModelingToolsEditorModeStyle::Get()->GetOptionalBrush(ActiveToolIconName);


	GetToolkitHost()->AddViewportOverlayWidget(ToolShutdownViewportOverlayWidget.ToSharedRef());

	// disable LOD level picker once Tool is active
	AssetLODMode->SetEnabled(false);
	AssetLODModeLabel->SetEnabled(false);

	// Invalidate all the level viewports so that e.g. hitproxy buffers are cleared
	// (fixes the editor gizmo still being clickable despite not being visible)
	if (GIsEditor)
	{
		for (FLevelEditorViewportClient* Viewport : GEditor->GetLevelViewportClients())
		{
			Viewport->Invalidate();
		}
	}
	RebuildPresetListForTool(false);
}

void FModelingToolsEditorModeToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	bInActiveTool = false;

	if (IsHosted())
	{
		GetToolkitHost()->RemoveViewportOverlayWidget(ToolShutdownViewportOverlayWidget.ToSharedRef());
	}

	ModeDetailsView->SetObject(nullptr);
	ActiveToolName = FText::GetEmpty();
	if (HasToolkitBuilder())
	{
		ToolkitBuilder->SetActiveToolDisplayName(FText::GetEmpty());		
		ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Visible);
	}

	ModeHeaderArea->SetVisibility(EVisibility::Visible);
	ModeHeaderArea->SetText(LOCTEXT("SelectToolLabel", "Select a Tool from the Toolbar"));
	ClearNotification();
	ClearWarning();
	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
	if ( CurTool )
	{
		CurTool->OnPropertySetsModified.RemoveAll(this);
		CurTool->OnPropertyModifiedDirectlyByTool.RemoveAll(this);
	}

	// re-enable LOD level picker
	AssetLODMode->SetEnabled(true);
	AssetLODModeLabel->SetEnabled(true);

	ClearPresetComboList();
}

void FModelingToolsEditorModeToolkit::OnActiveViewportChanged(TSharedPtr<IAssetViewport> OldViewport, TSharedPtr<IAssetViewport> NewViewport)
{
	// Only worry about handling this notification if Modeling has an active tool
	if (!ActiveToolName.IsEmpty())
	{
		// Check first to see if this changed because the old viewport was deleted and if not, remove our hud
		if (OldViewport)	
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(ToolShutdownViewportOverlayWidget.ToSharedRef(), OldViewport);

			if (GizmoNumericalUIOverlayWidget.IsValid())
			{
				GetToolkitHost()->RemoveViewportOverlayWidget(GizmoNumericalUIOverlayWidget.ToSharedRef(), OldViewport);
			}
		}

		// Add the hud to the new viewport
		GetToolkitHost()->AddViewportOverlayWidget(ToolShutdownViewportOverlayWidget.ToSharedRef(), NewViewport);

		if (GizmoNumericalUIOverlayWidget.IsValid())
		{
			GetToolkitHost()->AddViewportOverlayWidget(GizmoNumericalUIOverlayWidget.ToSharedRef(), NewViewport);
		}
	}

#if ENABLE_STYLUS_SUPPORT
	if (StylusInputHandler && NewViewport.IsValid())
	{
		StylusInputHandler->RegisterWindow(NewViewport->AsWidget());
	}
#endif
}




void FModelingToolsEditorModeToolkit::UpdateAssetLocationMode(TSharedPtr<FString> NewString)
{
	UModelingToolsEditorModeSettings* Settings = GetMutableDefault<UModelingToolsEditorModeSettings>();
	EModelingModeAssetGenerationLocation AssetGenerationLocation = EModelingModeAssetGenerationLocation::AutoGeneratedWorldRelativeAssetPath;
	if (NewString == AssetLocationModes[0])
	{
		AssetGenerationLocation = EModelingModeAssetGenerationLocation::AutoGeneratedWorldRelativeAssetPath;
	}
	else if (NewString == AssetLocationModes[1])
	{
		AssetGenerationLocation = EModelingModeAssetGenerationLocation::AutoGeneratedGlobalAssetPath;
	}
	else if (NewString == AssetLocationModes[2])
	{
		AssetGenerationLocation = EModelingModeAssetGenerationLocation::CurrentAssetBrowserPathIfAvailable;
	}

	Settings->SetAssetGenerationLocation(AssetGenerationLocation);
	
	Settings->SaveConfig();
}

void FModelingToolsEditorModeToolkit::UpdateAssetSaveMode(TSharedPtr<FString> NewString)
{
	UModelingToolsEditorModeSettings* Settings = GetMutableDefault<UModelingToolsEditorModeSettings>();
	EModelingModeAssetGenerationBehavior AssetGenerationBehavior = EModelingModeAssetGenerationBehavior::AutoGenerateButDoNotAutosave;
	if (NewString == AssetSaveModes[0])
	{
		AssetGenerationBehavior = EModelingModeAssetGenerationBehavior::AutoGenerateAndAutosave;
	}
	else if (NewString == AssetSaveModes[1])
	{
		AssetGenerationBehavior = EModelingModeAssetGenerationBehavior::AutoGenerateButDoNotAutosave;
	}
	else if (NewString == AssetSaveModes[2])
	{
		AssetGenerationBehavior = EModelingModeAssetGenerationBehavior::InteractivePromptToSave;
	}

	Settings->SetAssetGenerationMode(AssetGenerationBehavior);

	Settings->SaveConfig();
}

void FModelingToolsEditorModeToolkit::UpdateAssetPanelFromSettings()
{
	const UModelingToolsEditorModeSettings* Settings = GetDefault<UModelingToolsEditorModeSettings>();

	switch (Settings->GetAssetGenerationLocation())
	{
	case EModelingModeAssetGenerationLocation::CurrentAssetBrowserPathIfAvailable:
		AssetLocationMode->SetSelectedItem(AssetLocationModes[2]);
		break;
	case EModelingModeAssetGenerationLocation::AutoGeneratedGlobalAssetPath:
		AssetLocationMode->SetSelectedItem(AssetLocationModes[1]);
		break;
	case EModelingModeAssetGenerationLocation::AutoGeneratedWorldRelativeAssetPath:
	default:
		AssetLocationMode->SetSelectedItem(AssetLocationModes[0]);
		break;
	}

	AssetLocationMode->SetEnabled(!Settings->InRestrictiveMode());

	switch (Settings->GetAssetGenerationMode())
	{
	case EModelingModeAssetGenerationBehavior::AutoGenerateButDoNotAutosave:
		AssetSaveMode->SetSelectedItem(AssetSaveModes[1]);
		break;
	case EModelingModeAssetGenerationBehavior::InteractivePromptToSave:
		AssetSaveMode->SetSelectedItem(AssetSaveModes[2]);
		break;
	default:
		AssetSaveMode->SetSelectedItem(AssetSaveModes[0]);
		break;
	}
}


void FModelingToolsEditorModeToolkit::UpdateObjectCreationOptionsFromSettings()
{
	// update DynamicMeshActor Settings
	const UModelingToolsEditorModeSettings* Settings = GetDefault<UModelingToolsEditorModeSettings>();

	// enable/disable dynamic mesh actors
	UCreateMeshObjectTypeProperties::bEnableDynamicMeshActorSupport = true;

	// set configured default type
	if (Settings->DefaultMeshObjectType == EModelingModeDefaultMeshObjectType::DynamicMeshActor)
	{
		UCreateMeshObjectTypeProperties::DefaultObjectTypeIdentifier = UCreateMeshObjectTypeProperties::DynamicMeshActorIdentifier;
	}
	else if (Settings->DefaultMeshObjectType == EModelingModeDefaultMeshObjectType::VolumeActor)
	{
		UCreateMeshObjectTypeProperties::DefaultObjectTypeIdentifier = UCreateMeshObjectTypeProperties::VolumeIdentifier;
	}
	else
	{
		UCreateMeshObjectTypeProperties::DefaultObjectTypeIdentifier = UCreateMeshObjectTypeProperties::StaticMeshIdentifier;
	}
}

void FModelingToolsEditorModeToolkit::OnAssetSettingsModified()
{
	UpdateObjectCreationOptionsFromSettings();
	UpdateAssetPanelFromSettings();
}

void FModelingToolsEditorModeToolkit::OnShowAssetSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->ShowViewer("Project", "Plugins", "ModelingMode");
	}
}

void FModelingToolsEditorModeToolkit::SelectNewAssetPath() const
{	
	const FString PackageFolderPath = UE::Modeling::GetGlobalAssetRootPath();
	
	const TSharedPtr<SDlgPickPath> PickPathWidget =
		SNew(SDlgPickPath)
		.Title(LOCTEXT("ChooseAssetPathLabel", "Choose New Asset Path"))
		.DefaultPath(FText::FromString(PackageFolderPath + GetRestrictiveModeAutoGeneratedAssetPathText().ToString()))
		.AllowReadOnlyFolders(false);

	if (PickPathWidget->ShowModal() == EAppReturnType::Ok)
	{
		FString Path = PickPathWidget->GetPath().ToString();
		Path.RemoveFromStart(PackageFolderPath);	// Remove project folder prefix
		
		UModelingToolsEditorModeSettings* Settings = GetMutableDefault<UModelingToolsEditorModeSettings>();
		Settings->SetRestrictiveModeAutoGeneratedAssetPath(Path);
	}
}

void FModelingToolsEditorModeToolkit::UpdateCategoryButtonLabelVisibility(UObject* Obj, FPropertyChangedEvent& ChangeEvent)
{
	UModelingToolsModeCustomizationSettings* UISettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>();
	ToolkitBuilder->SetCategoryButtonLabelVisibility(UISettings != nullptr  ? UISettings->bShowCategoryButtonLabels : true);
	ToolkitBuilder->RefreshCategoryToolbarWidget();
}

void FModelingToolsEditorModeToolkit::UpdateSelectionColors(UObject* Obj, FPropertyChangedEvent& ChangeEvent) const
{
	if (CachedSelectionManager)
	{
		const UModelingToolsModeCustomizationSettings* UISettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>();
		CachedSelectionManager->SetSelectionColors(UISettings->UnselectedColor, UISettings->HoverOverSelectedColor, UISettings->HoverOverUnselectedColor, UISettings->GeometrySelectedColor, UISettings->GeometrySoftSelectedColor);
	}
}

FText FModelingToolsEditorModeToolkit::GetRestrictiveModeAutoGeneratedAssetPathText() const
{
	const UModelingToolsEditorModeSettings* Settings = GetDefault<UModelingToolsEditorModeSettings>();
	return FText::FromString(Settings->GetRestrictiveModeAutoGeneratedAssetPath());
}

void FModelingToolsEditorModeToolkit::OnRestrictiveModeAutoGeneratedAssetPathTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit) const
{
	UModelingToolsEditorModeSettings* Settings = GetMutableDefault<UModelingToolsEditorModeSettings>();
	Settings->SetRestrictiveModeAutoGeneratedAssetPath(InNewText.ToString());
}

void FModelingToolsEditorModeToolkit::NotifySelectionSystemEnabledStateModified()
{
	if (UModelingToolsEditorMode* ModelingMode = Cast<UModelingToolsEditorMode>(GetScriptableEditorMode()))
	{
		ModelingMode->NotifySelectionSystemEnabledStateModified();
	}
	
	//if (bUsesToolkitBuilder && ToolkitBuilder.IsValid())
	//{
	//	ToolkitBuilder->InitCategoryToolbarContainerWidget();
	//}
	//else
	//{
		// required due to above
		FNotificationInfo Info(LOCTEXT("ChangeSelectionEnabledToast",
			"You should exit and re-enter Modeling Mode after toggling Mesh Element Selection Support to fully update the UI"));
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	//}
}

bool FModelingToolsEditorModeToolkit::RequestAcceptCancelButtonOverride(IToolHostCustomizationAPI::FAcceptCancelButtonOverrideParams& Params)
{
	if (!Params.OnAcceptCancelTriggered || !Params.CanAccept || Params.Label.IsEmpty())
	{
		UE_LOGF(LogGeometry, Warning, "FModelingToolsEditorModeToolkit::RequestAcceptCancelButtonOverride received request with insufficient parameters.");
		return false;
	}

	AcceptCancelButtonParams = Params;
	CompleteButtonParams.Reset();
	bCurrentOverrideButtonsWereClicked = false;

	if (ToolShutdownViewportOverlayWidget)
	{
		ToolShutdownViewportOverlayWidget->Invalidate(EInvalidateWidgetReason::Layout);
	}
	return true;
}
bool FModelingToolsEditorModeToolkit::RequestCompleteButtonOverride(IToolHostCustomizationAPI::FCompleteButtonOverrideParams& Params)
{
	if (!Params.OnCompleteTriggered || Params.Label.IsEmpty())
	{
		UE_LOGF(LogGeometry, Warning, "FModelingToolsEditorModeToolkit::RequestCompleteButtonOverride received request with insufficient parameters.");
		return false;
	}

	CompleteButtonParams = Params;
	AcceptCancelButtonParams.Reset();
	bCurrentOverrideButtonsWereClicked = false;

	if (ToolShutdownViewportOverlayWidget)
	{
		ToolShutdownViewportOverlayWidget->Invalidate(EInvalidateWidgetReason::Layout);
	}
	return true;
}
void FModelingToolsEditorModeToolkit::ClearButtonOverrides()
{
	AcceptCancelButtonParams.Reset();
	CompleteButtonParams.Reset();
	if (ToolShutdownViewportOverlayWidget)
	{
		ToolShutdownViewportOverlayWidget->Invalidate(EInvalidateWidgetReason::Layout);
	}
}


FReply FModelingToolsEditorModeToolkit::HandleAcceptCancelClick(bool bAccept)
{
	if (AcceptCancelButtonParams.IsSet())
	{
		bCurrentOverrideButtonsWereClicked = true;
		if (ensure(AcceptCancelButtonParams->OnAcceptCancelTriggered))
		{
			AcceptCancelButtonParams->OnAcceptCancelTriggered(bAccept);
		}
		
		// This will be reset back to false if the callback above triggers another override request.
		if (bCurrentOverrideButtonsWereClicked)
		{
			ClearButtonOverrides();
		}
	}
	else
	{
		GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->EndTool(
			bAccept ? EToolShutdownType::Accept : EToolShutdownType::Cancel);
	}
	return FReply::Handled();
}
FReply FModelingToolsEditorModeToolkit::HandleCompleteClick()
{
	if (CompleteButtonParams.IsSet())
	{
		bCurrentOverrideButtonsWereClicked = true;
		if (ensure(CompleteButtonParams->OnCompleteTriggered))
		{
			CompleteButtonParams->OnCompleteTriggered();
		}

		// This will be reset back to false if the callback above triggers another override request.
		if (bCurrentOverrideButtonsWereClicked)
		{
			ClearButtonOverrides();
		}
	}
	else
	{
		GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->EndTool(EToolShutdownType::Completed);
	}

	return FReply::Handled();
}

void FModelingToolsEditorModeToolkit::DisconnectStylusStateProviderAPI()
{
#if ENABLE_STYLUS_SUPPORT
	if (StylusInputHandler)
	{
		// replace the input handler with a new, empty version -- disconnecting old windows/contexts
		StylusInputHandler = MakeUnique<UE::Modeling::FStylusInputHandler>();
	}
#endif
}

IToolStylusStateProviderAPI* FModelingToolsEditorModeToolkit::GetStylusStateProviderAPI() const
{
#if ENABLE_STYLUS_SUPPORT
	return StylusInputHandler.Get();
#else
	return nullptr;
#endif
}


bool FModelingToolsEditorModeToolkit::EngineAssetsSelected() const
{
	auto HasEngineAsset = [](UStaticMeshComponent* SMC) -> bool
	{
		UStaticMesh* StaticMesh = SMC->GetStaticMesh();
		return StaticMesh && StaticMesh->GetPathName().StartsWith(TEXT("/Engine/"));
	};
	bool bFoundEngineAssets = false;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		AActor* Actor = static_cast<AActor*>(*It);
		Actor->ForEachComponent<UStaticMeshComponent>(false, [&HasEngineAsset, &bFoundEngineAssets](UStaticMeshComponent* SMC)
		{
			bFoundEngineAssets = bFoundEngineAssets || HasEngineAsset(SMC);
		});
	}
	for (FSelectionIterator It(GEditor->GetSelectedComponentIterator()); It; ++It)
	{
		if (UStaticMeshComponent* SMC = Cast<UStaticMeshComponent>(*It))
		{
			bFoundEngineAssets = bFoundEngineAssets || HasEngineAsset(SMC);
		}
	}
	return bFoundEngineAssets;
}


#undef LOCTEXT_NAMESPACE
