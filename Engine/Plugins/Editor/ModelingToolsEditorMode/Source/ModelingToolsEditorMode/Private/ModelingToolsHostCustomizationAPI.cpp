// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingToolsHostCustomizationAPI.h"

#include "ContextObjectStore.h"
#include "InteractiveToolsContext.h"
#include "ModelingToolsEditorModeToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelingToolsHostCustomizationAPI)

bool UModelingToolsHostCustomizationAPI::RequestAcceptCancelButtonOverride(FAcceptCancelButtonOverrideParams& Params)
{
	if (TSharedPtr<FModelingToolsEditorModeToolkit> ToolkitPinned = Toolkit.Pin())
	{
		return ToolkitPinned->RequestAcceptCancelButtonOverride(Params);
	}
	return false;
}

bool UModelingToolsHostCustomizationAPI::RequestCompleteButtonOverride(FCompleteButtonOverrideParams& Params)
{
	if (TSharedPtr<FModelingToolsEditorModeToolkit> ToolkitPinned = Toolkit.Pin())
	{
		return ToolkitPinned->RequestCompleteButtonOverride(Params);
	}
	return false;
}

void UModelingToolsHostCustomizationAPI::ClearButtonOverrides()
{
	if (TSharedPtr<FModelingToolsEditorModeToolkit> ToolkitPinned = Toolkit.Pin())
	{
		ToolkitPinned->ClearButtonOverrides();
	}
}

UModelingToolsHostCustomizationAPI* UModelingToolsHostCustomizationAPI::Register(
	UInteractiveToolsContext* ToolsContext, TSharedRef<FModelingToolsEditorModeToolkit> Toolkit)
{
	if (!ensure(ToolsContext && ToolsContext->ContextObjectStore))
	{
		return nullptr;
	}

	UModelingToolsHostCustomizationAPI* Found = ToolsContext->ContextObjectStore->FindContext<UModelingToolsHostCustomizationAPI>();
	if (Found)
	{
		if (!ensureMsgf(Found->Toolkit == Toolkit,
			TEXT("UModelingToolsHostCustomizationAPI already registered, but with different toolkit. "
				"Do not expect multiple toolkits to provide tool overlays to customize.")))
		{
			Found->Toolkit = Toolkit;
		}
		return Found;
	}
	else
	{
		UModelingToolsHostCustomizationAPI* Instance = NewObject<UModelingToolsHostCustomizationAPI>();
		Instance->Toolkit = Toolkit;
		ensure(ToolsContext->ContextObjectStore->AddContextObject(Instance));
		return Instance;
	}
}

bool UModelingToolsHostCustomizationAPI::Deregister(UInteractiveToolsContext* ToolsContext)
{
	if (!ensure(ToolsContext && ToolsContext->ContextObjectStore))
	{
		return false;
	}

	ToolsContext->ContextObjectStore->RemoveContextObjectsOfType(UModelingToolsHostCustomizationAPI::StaticClass());
	return true;
}
