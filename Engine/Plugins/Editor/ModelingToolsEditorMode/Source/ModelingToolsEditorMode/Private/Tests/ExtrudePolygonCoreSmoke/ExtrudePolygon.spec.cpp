// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Engine/World.h"
#include "Engine/StaticMeshActor.h"
#include "Components/StaticMeshComponent.h"
#include "Misc/AutomationTest.h"
#include "EngineUtils.h"
#include "ObjectTools.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "ModelingToolsEditorMode.h"
#include "DrawPolygonTool.h"

BEGIN_DEFINE_SPEC(
	FExtrudePolygonSpec, "Editor.Plugins.Tools.Modeling.EditorMode",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	UEdMode* EditorMode;
	UInteractiveToolManager* InteractiveToolManager;
	UDrawPolygonTool* DrawPolygonTool;
	UWorld* World;
	AStaticMeshActor* PolygonMesh;
	// Test helper functions
	void CreateMesh() const;
	bool MeshExists(const FString&);
	bool ModelingSettingsExist(const FString&) const;

END_DEFINE_SPEC(FExtrudePolygonSpec)
void FExtrudePolygonSpec::Define()
{
	Describe("ExtrudePolygon", [this]()
	{
		BeforeEach([this]()
		{
			// Entering Extrude Polygon Tool
			GLevelEditorModeTools().ActivateMode(UModelingToolsEditorMode::EM_ModelingToolsEditorModeId);
			EditorMode = GLevelEditorModeTools().GetActiveScriptableMode(UModelingToolsEditorMode::EM_ModelingToolsEditorModeId);
			if (EditorMode)
			{
				InteractiveToolManager = EditorMode->GetToolManager();
				World = EditorMode->GetWorld();
			}
			if (InteractiveToolManager)
			{
				InteractiveToolManager->SelectActiveToolType(EToolSide::Left, TEXT("BeginDrawPolygonTool"));
				InteractiveToolManager->ActivateTool(EToolSide::Left);
				DrawPolygonTool = Cast<UDrawPolygonTool>(InteractiveToolManager->GetActiveTool(EToolSide::Left));
			}
		});

		It("Should create Polygon Mesh using Extrude Polygon Tool", [this]()
		{
			CreateMesh();

			// Test requires that Static Mesh created should be named "Extrude"
			const FString PolygonName = TEXT("Extrude");

			const bool bPolygonExists = MeshExists(PolygonName);

			TestTrue(TEXT("StaticMeshActor named \"Extrude\" exists"), bPolygonExists);
		});

		It("Should validate proper removal and addition of Polygon Mesh via undo-redo", [this]()
		{
			CreateMesh();

			// Test requires that Static Mesh created should be named "Extrude"
			const FString PolygonName = TEXT("Extrude");

			GEditor->UndoTransaction();
			const bool bMeshUndoState = MeshExists(PolygonName);
			
			if (TestTrue(TEXT("Polygon Mesh absent after Undo"), !bMeshUndoState))
			{
				// Proceed with Redo only if the Undo operation was successful
				GEditor->RedoTransaction();
				const bool bMeshRedoState = MeshExists(PolygonName);
				TestTrue(TEXT("Polygon Mesh restored after Redo"), bMeshRedoState);
			}
		});

		It("Should dismiss Extrude Polygon Settings after creating a vertex and completing", [this]()
		{
			const FString ExtrudePolygonToolName = TEXT("BeginDrawPolygonTool");
			// Checking that during setup we entered Extrude Polygon Tool
			if (!TestTrue(TEXT("Entered Extrude Polygon Tool successfully"), ModelingSettingsExist(ExtrudePolygonToolName)))
			{
				return;
			}

			if (InteractiveToolManager && DrawPolygonTool)
			{
				DrawPolygonTool->AppendVertex(FVector3d::ZeroVector);
				// Clicking "Complete" will invoke InteractiveToolManager->DeactivateTool
				InteractiveToolManager->DeactivateTool(EToolSide::Left, EToolShutdownType::Completed);
			}

			TestTrue(TEXT("Extrude Polygon Settings dismissed after completing"), !ModelingSettingsExist(ExtrudePolygonToolName));
		});

		It("Should populate Modeling tab with Extrude Polygon settings after tool entry", [this]()
		{
			const FString ExtrudePolygonToolName = TEXT("BeginDrawPolygonTool");
			TestTrue(TEXT("The Modeling tab is populated with Extrude Polygon settings"),
					 ModelingSettingsExist(ExtrudePolygonToolName));
		});

		AfterEach([this]()
		{
			// Grabbing Static Mesh of PolygonMesh before we remove the actor
			UStaticMesh* StaticMesh = nullptr;
			if (PolygonMesh)
			{
				if (const UStaticMeshComponent* StaticMeshComponent = PolygonMesh->GetStaticMeshComponent())
				{
					StaticMesh = StaticMeshComponent->GetStaticMesh();
				}
				// Removing PolygonMesh Actor
				World->EditorDestroyActor(PolygonMesh, false);
			}
			// Cleaning up temporary Static Mesh that doesn't get removed by removing the Actor
			if (StaticMesh)
			{
				TArray<UObject*> AssetsToDelete;
				AssetsToDelete.Add(StaticMesh);
				ObjectTools::DeleteObjects(AssetsToDelete, false, ObjectTools::EAllowCancelDuringDelete::CancelNotAllowed);
			}
			// Resetting the interface out of Modeling Tools
			InteractiveToolManager->DeactivateTool(EToolSide::Left, EToolShutdownType::Completed);
			GLevelEditorModeTools().ActivateMode(FBuiltinEditorModes::EM_Default);
		});
	});
}

void FExtrudePolygonSpec::CreateMesh() const
{
	// We use 3 arbitrary vertices, based on which we'll create the Polygon Mesh
	const FVector3d VertexA(0.0f, 0.0f, 0.0f);
	const FVector3d VertexB(0.0f, 100.0f, 0.0f);
	const FVector3d VertexC(100.0f, 0.0f, 0.0f);

	if (DrawPolygonTool)
	{
		DrawPolygonTool->AppendVertex(VertexA);
		DrawPolygonTool->AppendVertex(VertexB);
		DrawPolygonTool->AppendVertex(VertexC);
		DrawPolygonTool->EmitCurrentPolygon();
	}
}

bool FExtrudePolygonSpec::MeshExists(const FString& PolygonName)
{
	bool bMeshExists = false;

	if (World)
	{
		for (AActor* Actor : TActorRange<AActor>(World))
		{
			AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>(Actor);
			if (StaticMeshActor && StaticMeshActor->GetActorLabel() == PolygonName)
			{
				bMeshExists = true;
				PolygonMesh = StaticMeshActor;
			}
		}
	}

	return bMeshExists;
}

bool FExtrudePolygonSpec::ModelingSettingsExist(const FString& ToolName) const
{
	const FString ActiveToolName = InteractiveToolManager->GetActiveToolName(EToolSide::Left);
	if (ActiveToolName == ToolName)
	{
		return true;
	}
	return false;
}
