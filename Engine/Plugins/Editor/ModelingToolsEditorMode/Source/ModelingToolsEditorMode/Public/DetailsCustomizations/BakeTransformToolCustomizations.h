// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IPropertyHandle;
class FPropertyRestriction;

// Customization for UBakeTransformToolProperties
class FBakeTransformToolDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	void EvaluateBakeScaleMethodRestrictions(TSharedRef<IPropertyHandle> AllowNoScale, TSharedRef<IPropertyHandle> BakeScaleHandle);

	/** Property restriction applied to the EBakeScaleMethod enum dropdown box */
	TSharedPtr<FPropertyRestriction> BakeScaleMethodEnumRestriction;
};
