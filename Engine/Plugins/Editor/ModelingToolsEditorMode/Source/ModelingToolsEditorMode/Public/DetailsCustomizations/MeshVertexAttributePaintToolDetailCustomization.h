// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class IDetailLayoutBuilder;
class IDetailCategoryBuilder;
class FSlimHorizontalToolBarBuilder;
class SWidget;
class UMeshVertexAttributePaintToolBase;
class UMeshVertexAttributePaintToolProperties;

enum class EMeshVertexAttributePaintToolValueQueryType : uint8;

namespace UE::MeshVertexAttributePaintToolBase
{
	class FVertexAttributePaintToolDetailCustomization : public IDetailCustomization
	{
	public:
		FVertexAttributePaintToolDetailCustomization();
		virtual ~FVertexAttributePaintToolDetailCustomization() override;

		static TSharedRef<IDetailCustomization> MakeInstance();

		/** IDetailCustomization interface */
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

		void OnSelectionChanged();

	private:

		void AddEditingModeRow(IDetailCategoryBuilder& EditModeCategory) const;
		void AddColorModeRow(IDetailCategoryBuilder& EditModeCategory) const;
		void AddColorRampRow(IDetailCategoryBuilder& EditModeCategory) const;
		TSharedRef<SWidget> BuildColorRampMenu() const;
		void AddValuesDisplayRows(IDetailCategoryBuilder& EditModeCategory) const;

		void AddBrushUI(IDetailLayoutBuilder& DetailBuilder) const;
		void AddBrushModeRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddBrushFalloffModeRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddBrushRadiusRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddBrushValueRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddBrushFalloffRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddBrushAccumulateRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddBrushValueAtBrushRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddBrushValueQueryMethodRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddBrushHitBackFacesRow(IDetailCategoryBuilder& BrushCategory) const;

		TSharedRef<SWidget> MakeBrushMirrorEnableWidget() const;
		void AddBrushMirrorEnableRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddBrushMirrorPlaneRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddBrushMirrorHideOnBrushStrokeRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddBrushMirrorObjectSpaceRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddBrushMirrorUI(IDetailLayoutBuilder& DetailBuilder) const;

		void AddVisibilityFilterRow(IDetailCategoryBuilder& BrushCategory) const;

		void AddSelectionUI(IDetailLayoutBuilder& DetailBuilder) const;
		TSharedRef<SWidget> MakeDragToolToolbar() const;
		TSharedRef<SWidget> MakeSelectionElementsToolbar() const;
		TSharedRef<SWidget> MakeSelectionIsolationWidget() const;
		TSharedRef<SWidget> MakeSelectionEditActionsToolbar() const;
		void AddSelectionElementsRow(IDetailCategoryBuilder& BrushCategory) const;
		void AddEmptySelectionWarningRow(IDetailCategoryBuilder& BrushCategory) const;
		void MakeSelectionOperationRow(IDetailCategoryBuilder& EditValuesCategory, const FText& RowName, const TSharedRef<SWidget>& ButtonWidget, const TSharedRef<SWidget>& ValueWidget) const;
		void MakeSelectionAddMultiplySliderOperationRow(IDetailCategoryBuilder& EditValuesCategory) const;
		void MakeSelectionAddOperationRow(IDetailCategoryBuilder& EditValuesCategory) const;
		void MakeSelectionReplaceOperationRow(IDetailCategoryBuilder& EditValuesCategory) const;
		void MakeSelectionInvertOperationRow(IDetailCategoryBuilder& EditValuesCategory) const;
		void MakeSelectionRelaxOperationRow(IDetailCategoryBuilder& EditValuesCategory) const;
		void MakeSelectionMirrorOperationRow(IDetailCategoryBuilder& EditValuesCategory) const;
		void MakeSelectionPruneOperationRow(IDetailCategoryBuilder& EditValuesCategory) const;
		void MakeSelectionCopyAndPasteRow(IDetailCategoryBuilder& EditValuesCategory) const;
		
		TSharedRef<SWidget> MakeMirrorAxisWidget() const;
		TSharedRef<SWidget> MakeMirrorShowAndAxisWidget() const;
		TSharedRef<SWidget> MakeMirrorDirectionButtonsWidget() const;
		TSharedRef<SWidget> MakeMirrorPlaneWidget(bool bShowDirection) const;
		TSharedRef<SWidget> MakeMirrorObjectSpaceCheckboxWidget() const;
		TSharedRef<SWidget> MakeMirrorObjectSpaceWidget() const;
		TSharedRef<SWidget> MakeMirrorButton() const;

		void HideProperty(IDetailLayoutBuilder& DetailBuilder, FName PropertyName) const;
		
		IDetailLayoutBuilder* CurrentDetailBuilder = nullptr;
		TWeakObjectPtr<UMeshVertexAttributePaintToolProperties> ToolProperties;
		TWeakObjectPtr<UMeshVertexAttributePaintToolBase> Tool;

		TArray<FName> BrushQueryTypeNames;
		TMap<FName, EMeshVertexAttributePaintToolValueQueryType> BrushQueryTypeEnumFromNames;
		TMap<EMeshVertexAttributePaintToolValueQueryType, FName> BrushQueryTypeNameFromEnum;

		TArray<FName> VisibilityFilterNames;

		static float WeightSliderWidths;
		static float WeightEditingLabelsPercent;
		static float WeightEditVerticalPadding;
		static float WeightEditHorizontalPadding;
	};
}