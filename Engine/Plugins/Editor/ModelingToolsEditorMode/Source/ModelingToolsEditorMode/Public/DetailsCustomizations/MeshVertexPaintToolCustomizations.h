// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class IPropertyHandle;

// customization for UVertexPaintBasicProperties
// creates custom widgets for paint and erase brushes with pressure sensitivity
class FVertexPaintBasicPropertiesDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
private:
	// helper to build strength widget, with pressure sensitivity toggle
	static void BuildPaintPressureWidget(
		IDetailLayoutBuilder& DetailBuilder,
		const TSharedPtr<IPropertyHandle>& ColorPropHandle,
		TSharedPtr<IPropertyHandle> EnablePressureSensitivityHandle
	);
};