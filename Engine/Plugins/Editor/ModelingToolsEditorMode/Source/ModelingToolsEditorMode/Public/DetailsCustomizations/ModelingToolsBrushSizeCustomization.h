// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IDetailLayoutBuilder;

// defining this here so we can use in FModelingToolsBrushSizeCustomization, should move to some shared constants header/file
class FSculptToolsUIConstants
{
public:
	// sculpt panel packs in the widgets w/ short fixed-width labels
	static constexpr int SculptShortLabelWidth = 40;

};

// Details customization for FBrushToolRadius struct.
// This maybe should be removed in favor of just doing custom UI in FSculptBrushPropertiesDetails...
class FModelingToolsBrushSizeCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
};
