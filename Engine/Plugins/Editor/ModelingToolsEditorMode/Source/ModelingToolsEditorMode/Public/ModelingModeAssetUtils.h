// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class FString;
class UObject;

class UWorld;


namespace UE
{
namespace Modeling
{

//
// Utility functions that can be used to plug in the client-configurable parts of UEditorModelingObjectsCreationAPI.
// See ModelingToolsEditorMode for usage.
//

/**
 * Determines path and name for a new Asset based on current mode settings, etc
 * @param BaseName desired initial string for the name
 * @param TargetWorld World the new Asset will be used in, which can determine its generated path based on various settings
 * @param SuggestedFolder a suggested path for the asset, ignored if empty
 * @return Path+Name for the new Asset (package)
 */
MODELINGTOOLSEDITORMODE_API FString GetNewAssetPathName(const FString& BaseName, const UWorld* TargetWorld, FString SuggestedFolder);

/**
 * Determines the world relative asset root path.
 * @param World World the new Asset will be used in; if this is a nullptr then it will be set using the current world context.
 * @return Asset root path relative to the world.
 */
MODELINGTOOLSEDITORMODE_API FString GetWorldRelativeAssetRootPath(const UWorld* World);

/**
 * Determines the global asset root path.
 * @return Global asset root path.
 */
MODELINGTOOLSEDITORMODE_API FString GetGlobalAssetRootPath();

/**
 * Utility function that may auto-save a newly created asset depending on current mode settings,
 * and posts notifications for the Editor
 * @param Asset the new asset object (eg UStaticMesh, UTexture2D, etc)
 */
MODELINGTOOLSEDITORMODE_API void OnNewAssetCreated(UObject* Asset);


/**
 * Utility function that will attempt to auto-save an asset, and post Editor change notifications
 * @param Asset the  asset object (eg UStaticMesh, UTexture2D, etc)
 */
MODELINGTOOLSEDITORMODE_API bool AutoSaveAsset(UObject* Asset);

}
}
