// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Internationalization/Text.h"
#include "ToolTargets/ToolTarget.h"

class UInteractiveToolsContext;
class FUICommandInfo;
class UInteractiveToolBuilder;
class IToolStylusStateProviderAPI;
namespace UE
{
	class IInteractiveToolCommandsInterface;
}


// dummy class for extension API below
class FModelingModeAssetAPI
{
public:
};


/**
 * This struct is passed to IModelingModeToolExtension implementations to allow 
 * them to forward various Tools Context information to their ToolBuilders/etc
 * 
 * Note that if bIsInfoQueryOnly is true, then the Extension does not need to 
 * construct Tool Builders, the query is for information purposes only
 */
struct FExtensionToolQueryInfo
{
	bool bIsInfoQueryOnly = false;

	UInteractiveToolsContext* ToolsContext = nullptr;
	FModelingModeAssetAPI* AssetAPI = nullptr;
	IToolStylusStateProviderAPI* StylusAPI = nullptr;
};

/**
 * IModelingModeToolExtension implementations return the list of Tools they provide
 * via instances of FExtensionToolDescription.
 */
struct FExtensionToolDescription
{
	/** Long name of the Tool, used in various places in the UI */
	FText ToolName;
	/** Command that is added to the Tool button set. This defines the button label. */
	TSharedPtr<FUICommandInfo> ToolCommand;
	/** Builder for the Tool. This can be null if FExtensionToolQueryInfo.bIsInfoQueryOnly is true. */
	UInteractiveToolBuilder* ToolBuilder = nullptr;
	/**
	 * Optional lambda that returns this tool's hotkey commands object, if one exists, so that
	 *  those commands can be properly bound/unbound when tool starts/ends. E.g. this will be
	 *  something like "return FThisExtensionToolCommands::Get();"
	 */
	TFunction<const UE::IInteractiveToolCommandsInterface&()> ToolCommandsGetter = nullptr;
};

/**
 * IModelingModeToolExtension implementations can optionally return additional information used in the Eidtor UI
 * via instances of FModelingModeExtensionExtendedInfo.
 */
struct FModelingModeExtensionExtendedInfo
{
	/** Tooltip to use for UI buttons that refer to the Extension */
	FText ToolPaletteButtonTooltip;
	/** Command button that will be used for the extension in the Modeling Mode Tab Bar. This can be undefined, in which case a Command button w/ default icon will be created. */
	TSharedPtr<FUICommandInfo> ExtensionCommand;
};

/**
 * IModelingModeToolExtension uses the IModularFeature API to allow a Plugin to provide
 * a set of InteractiveTool's to be exposed in Modeling Mode. The Tools will be 
 * included in a section of the Modeling Mode tool list, based on GetToolSectionName().
 * 
 */
class IModelingModeToolExtension : public IModularFeature
{
public:
	virtual ~IModelingModeToolExtension() = default;

	/**
	 * @return A text string that identifies this Extension.
	 */
	virtual FText GetExtensionName() = 0;

	/**
	 * @return A text string that defines the name of the Toolbar Section this Extension's tools will be included in.
	 * @warning If the same Section is used in multiple Extensions, some buttons may not be shown
	 */
	virtual FText GetToolSectionName() = 0;

	/**
	 * Query the Extension for a list of Tools to expose in Modeling Mode.
	 * Note that this function *will* be called multiple times by Modeling Mode, as the
	 * information about the set of Tools is needed in multiple places. The QueryInfo.bIsInfoQueryOnly
	 * flag indicates whether the caller requires ToolBuilder instances. 
	 * 
	 * If creating multiple copies of the ToolBuilder for a particular Tool would be problematic, it 
	 * is the responsiblity of the IModelingModeToolExtension implementation to cache these internally, 
	 * otherwise they will be garbage collected when the caller releases them.
	 */
	virtual void GetExtensionTools(const FExtensionToolQueryInfo& QueryInfo, TArray<FExtensionToolDescription>& ToolsOut) = 0;

	/**
	 * Query the Extension for extended information. This function is optional, the results will
	 * be ignored unless it is overridden and returns true.
	 */
	virtual bool GetExtensionExtendedInfo(FModelingModeExtensionExtendedInfo& InfoOut) { return false; }

	/** 
	 * Query the extension for additional tool targets. This function is optional and the result will be
	 * ignored unless it is overridden and returns true.
	 */
	virtual bool GetExtensionToolTargets(TArray<TSubclassOf<UToolTargetFactory>>& ToolTargetFactoriesOut) { return false; }

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("ModelingModeToolExtension"));
		return FeatureName;
	}
};
