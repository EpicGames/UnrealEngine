// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "InputBehaviorSet.h"
#include "FrameTypes.h"
#include "UObject/Package.h"
#include "ModelingSelectionInteraction.generated.h"

#define UE_API MODELINGTOOLSEDITORMODE_API

class UCombinedTransformGizmo;
class URectangleMarqueeInteraction;
class UTransformProxy;
namespace UE::Geometry { struct FGeometrySelectionUpdateConfig; }
struct FCameraRectangle;

class UGeometrySelectionManager;
class USingleClickInputBehavior;
class USingleClickOrDragInputBehavior;
class UMouseHoverBehavior;
class URectangleMarqueeMechanic;
class UPathSelectionInteraction;
class FCanvas;
class IToolsContextRenderAPI;
class UDragAlignmentInteraction;
class UKeyAsModifierInputBehavior;


UENUM()
enum class EModelingSelectionInteraction_DragMode : uint8
{
	NoDragInteraction = 0,
	PathInteraction = 1,
	RectangleMarqueeInteraction = 2
};

UENUM()
enum class EModelingSelectionInteraction_LocalFrameMode : uint8
{
	FromGeometry = 0,
	FromObject = 1,
};

/**
 * UModelingSelectionInteraction provides element-level selection behavior (ie mesh triangles/edges/vertices)
 * via the UGeometrySelectionManager. The Interaction doesn't actually know about the meshes or interact
 * with them at all, it simply "operates" the SelectionManager based on user input. 
 * 
 * In addition to handling selection/deselection behaviors based on user input, the UModelingSelectionInteraction
 * also creates a Gizmo for the active selection if it is transformable, and forwards gizmo transformations
 * to the SelectionManager.
 * 
 * Currently some desired Selection-related interactions require a slightly convoluted path through the code,
 * because (for example) we want the standard UE editor gizmo to have "click precedence" over our selection
 * click behaviors. However we will get the input event first, so we need to be able to check if the gizmo is
 * hit via a callback function provided by the EdMode owning the ToolsContext this Interaction will be registered in.
 * Similarly we need to be able to filter out hits if a closer visible object is hit, because there is
 * no higher-level Behavior that will do that for us. (This may be improved in future, if more of the existing
 * Editor interaction behaviors are re-implemented as InputBehaviors)
 * 
 */
UCLASS(MinimalAPI)
class UModelingSelectionInteraction : public UObject, public IInputBehaviorSource, public IClickBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:

	/**
	 * Set up the Interaction, creates and registers Behaviors/etc. 
	 * @param CanChangeSelectionCallbackIn this function will be called to determine if the current Selection is allowed to be modified (for example, when a Tool is active, we may wish to lock selection)
	 */
	virtual void Initialize(
		TObjectPtr<UGeometrySelectionManager> SelectionManager,
		TUniqueFunction<bool()> CanChangeSelectionCallbackIn,
		TUniqueFunction<bool(const FInputDeviceRay&)> ExternalHitCaptureCallbackIn);

	virtual void Shutdown();

	UGeometrySelectionManager* GetSelectionManager() { return SelectionManager; }
	UE::Geometry::FGeometrySelectionUpdateConfig GetActiveSelectionUpdateConfig() const;

	virtual void Render(IToolsContextRenderAPI* RenderAPI);
	virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);


	//
	// Support for drag-style interactions
	//

	EModelingSelectionInteraction_DragMode GetActiveDragMode() const { return ActiveDragMode; }
	UE_API void SetActiveDragMode(EModelingSelectionInteraction_DragMode NewMode);

	//
	// Support for transformation of selected geometry
	//

	// this needs to be called from EdMode tick or rendering, to avoid mouse-interrupt priority issues
	virtual void ApplyPendingTransformInteractions();

	DECLARE_MULTICAST_DELEGATE(FOnBeginTransformInteraction);
	FOnBeginTransformInteraction OnTransformBegin;

	DECLARE_MULTICAST_DELEGATE(FOnEndTransformInteraction);
	FOnEndTransformInteraction OnTransformEnd;

	//
	// Support for Adjusting Local Frame Mode
	//
	EModelingSelectionInteraction_LocalFrameMode GetLocalFrameMode() const { return LocalFrameMode; }
	UE_API void SetLocalFrameMode(EModelingSelectionInteraction_LocalFrameMode NewLocalFrameMode);

protected:
	// click-to-select behavior, various drag behaviors
	UPROPERTY()
	TObjectPtr<USingleClickOrDragInputBehavior> ClickOrDragBehavior;

	// mouse hover behavior
	UPROPERTY()
	TObjectPtr<UMouseHoverBehavior> HoverBehavior;

	// set of all behaviors, will be passed up to UInputRouter
	UPROPERTY()
	TObjectPtr<UInputBehaviorSet> BehaviorSet;

public:
	//
	// IInputBehaviorSource API
	//
	virtual const UInputBehaviorSet* GetInputBehaviors() const
	{
		return BehaviorSet;
	}

	//
	// IClickBehaviorTarget implementation
	//
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	virtual void OnClicked(const FInputDeviceRay& ClickPos) override;


	//
	// IHoverBehaviorTarget implementation
	//
	virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override;

	//
	// IModifierToggleBehaviorTarget implementation
	//
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;


protected:

	// default change-selection callback always allows selection change
	TUniqueFunction<bool()> CanChangeSelectionCallback = []() { return true; };

	// 
	TUniqueFunction<bool(const FInputDeviceRay&)> ExternalHitCaptureCallback = [](const FInputDeviceRay&) { return false; };

	void ComputeSceneHits(const FInputDeviceRay& ClickPos,
		bool& bHitActiveObjects, FInputRayHit& ActiveObjectHit,
		bool& bHitInactiveObjectFirst, FInputRayHit& InactiveObjectHit);

	bool bClearSelectionOnClicked = false;

	// flags used to identify behavior modifier keys/buttons
	static const int AddToSelectionModifier = 1;
	bool bAddToSelectionEnabled = false;

	static const int ToggleSelectionModifier = 2;
	bool bToggleSelectionEnabled = false;



	UPROPERTY()
	TObjectPtr<UGeometrySelectionManager> SelectionManager;

	FDelegateHandle OnSelectionModifiedDelegate;
	void OnSelectionManager_SelectionModified();




	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo;

	UPROPERTY()
	TObjectPtr<UDragAlignmentInteraction> DragAlignmentInteraction;

	UPROPERTY()
	TObjectPtr<UKeyAsModifierInputBehavior> DragAlignmentToggleBehavior;

	UE::Geometry::FFrame3d InitialGizmoFrame;
	FVector3d InitialGizmoScale;
	UE::Geometry::FFrame3d LastUpdateGizmoFrame;
	FVector3d LastUpdateGizmoScale;
	FVector3d LastTranslationDelta;
	FVector4d LastRotateDelta;
	FVector3d LastScaleDelta;
	bool bInActiveTransform = false;

	void UpdateGizmoOnSelectionChange();
	bool bGizmoUpdatePending = false;

	void OnBeginGizmoTransform(UTransformProxy* Proxy);
	void OnEndGizmoTransform(UTransformProxy* Proxy);
	void OnGizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform);


protected:
	EModelingSelectionInteraction_DragMode ActiveDragMode = EModelingSelectionInteraction_DragMode::NoDragInteraction;
	void UpdateActiveDragMode();

	EModelingSelectionInteraction_LocalFrameMode LocalFrameMode = EModelingSelectionInteraction_LocalFrameMode::FromGeometry;

	UPROPERTY()
	TObjectPtr<URectangleMarqueeInteraction> RectangleMarqueeInteraction;

	void OnMarqueeRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled);

	UPROPERTY()
	TObjectPtr<UPathSelectionInteraction> PathSelectionInteraction;
};



/**
 * UPathSelectionInteraction is a simple drag-interaction for Selection, which essentially just
 * selects/deselects any element hit by the cursor. 
 * 
 * Currently no attempt is made to (eg) subsample, so a fast-moving cursor will skip over some faces.
 */
UCLASS()
class UPathSelectionInteraction : public UObject, public IClickDragBehaviorTarget
{
	GENERATED_BODY()
public:
	void Setup(UModelingSelectionInteraction* SelectionInteraction);

protected:
	UPROPERTY()
	TWeakObjectPtr<UModelingSelectionInteraction> SelectionInteraction = nullptr;

	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	virtual void OnTerminateDragSequence() override;

};

#undef UE_API