// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/LegacyEdModeWidgetHelpers.h"

#include "ModelingModeToolExtensions.h" // FExtensionToolDescription

#include "ModelingToolsEditorMode.generated.h"

namespace UE
{
	class IInteractiveToolCommandsInterface;
}
class IToolsContextRenderAPI;
enum class EModelingModeActionCommands;
enum class EToolSide;
struct FInputDeviceRay;
struct FToolBuilderState;

class FEditorComponentSourceFactory;
class FUICommandList;
class FLevelObjectsObserver;
class UModelingSceneSnappingManager;
class UModelingSelectionInteraction;
class UGeometrySelectionManager;
class UInteractiveCommand;
class UBlueprint;

UCLASS(Transient, MinimalAPI)
class UModelingToolsEditorMode : public UBaseLegacyWidgetEdMode, public ILegacyEdModeSelectInterface
{
	GENERATED_BODY()
public:
	MODELINGTOOLSEDITORMODE_API const static FEditorModeID EM_ModelingToolsEditorModeId;

	UModelingToolsEditorMode();
	UModelingToolsEditorMode(FVTableHelper& Helper);
	~UModelingToolsEditorMode();
	////////////////
	// UEdMode interface
	////////////////

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;

	virtual void ActorSelectionChangeNotify() override;

	virtual bool ShouldDrawWidget() const override;
	virtual bool ProcessEditDelete() override;
	virtual bool ProcessEditCut() override;

	virtual bool CanAutoSave() const override;

	virtual bool GetPivotForOrbit(FVector& OutPivot) const override;
	
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	/*
	 * focus events
	 */

	// called when we "start" this editor mode (ie switch to this tab)
	virtual void Enter() override;

	// called when we "end" this editor mode (ie switch to another tab)
	virtual void Exit() override;

	virtual bool ShouldToolStartBeAllowed(const FString& ToolIdentifier) const override;

	//////////////////
	// End of UEdMode interface
	//////////////////


	// ILegacyEdModeSelectInterface
	virtual bool BoxSelect(FBox& InBox, bool InSelect = true) override;
	virtual bool FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect) override;


	// Manage viewport focus
	virtual bool HasCustomViewportFocus() const override;
	virtual FBox ComputeCustomViewportFocus() const override;



	//
	// Selection System configuration, this will likely move elsewhere
	//

	virtual UGeometrySelectionManager* GetSelectionManager() const
	{
		return SelectionManager;
	}
	virtual UModelingSelectionInteraction* GetSelectionInteraction() const
	{
		return SelectionInteraction;
	}

	UPROPERTY()
	bool bEnableVolumeElementSelection = true;

	UPROPERTY()
	bool bEnableStaticMeshElementSelection = true;

	bool GetMeshElementSelectionSystemEnabled() const;
	void NotifySelectionSystemEnabledStateModified();

protected:
	virtual void BindCommands() override;
	virtual void CreateToolkit() override;
	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	
	virtual void OnToolPostBuild(UInteractiveToolManager* InToolManager, EToolSide InSide, UInteractiveTool* InBuiltTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& ToolState);

	// Method to optionally register the UV Editor launcher in the Modeling Mode UV category if the plugin is available.
	void RegisterUVEditor();

	FDelegateHandle MeshCreatedEventHandle;
	FDelegateHandle TextureCreatedEventHandle;
	FDelegateHandle MaterialCreatedEventHandle;
	FDelegateHandle SelectionModifiedEventHandle;

	FDelegateHandle EditorClosedEventHandle;
	void OnEditorClosed();

	TSharedPtr<FLevelObjectsObserver> LevelObjectsObserver;

	UPROPERTY()
	TObjectPtr<UModelingSceneSnappingManager> SceneSnappingManager;

	UPROPERTY()
	TObjectPtr<UGeometrySelectionManager> SelectionManager;

	UPROPERTY()
	TObjectPtr<UModelingSelectionInteraction> SelectionInteraction;

	FDelegateHandle SelectionManager_SelectionModifiedHandle;

	bool GetGeometrySelectionChangesAllowed() const;
	bool TestForEditorGizmoHit(const FInputDeviceRay&) const;


	bool bSelectionSystemEnabled = false;

	void UpdateSelectionManagerOnEditorSelectionChange(bool bEnteringMode = false);

	void OnToolsContextRender(IToolsContextRenderAPI* RenderAPI);
	void OnToolsContextDrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);

	void ModelingModeShortcutRequested(EModelingModeActionCommands Command);
	void FocusCameraAtCursorHotkey();

	void AcceptActiveToolActionOrTool();
	void CancelActiveToolActionOrTool();

	void ConfigureRealTimeViewportsOverride(bool bEnable);

	FDelegateHandle BlueprintPreCompileHandle;
	void OnBlueprintPreCompile(UBlueprint* Blueprint);


	// UInteractiveCommand support. Currently implemented by creating instances of
	// commands on mode startup and holding onto them. This perhaps should be revisited,
	// command instances could probably be created as needed...

	UPROPERTY()
	TArray<TObjectPtr<UInteractiveCommand>> ModelingModeCommands;


	// analytics tracking
	static FDateTime LastModeStartTimestamp;
	static FDateTime LastToolStartTimestamp;

	// tracking of unlocked stuff
	static FDelegateHandle GlobalModelingWorldTeardownEventHandle;
private:
	// add modeling-mode-specific portion of new viewport toolbar
	void PopulateModelingModeViewportToolbar(const FName InMenuName, const TSharedPtr<const FUICommandList>& InCommandList);
	// remove modeling-mode-specific portion of new viewport toolbar
	static void RemoveModelingModeViewportToolbarExtensions();

	// Support extension tools having their own hotkey classes
	TMap<FString, FExtensionToolDescription> ExtensionToolToInfo;
	// Note: this will only work when the given tool is active, because we get the tool identifier
	//  out of the manager using GetActiveToolName
	bool TryGetExtensionToolCommandGetter(UInteractiveToolManager* Manager, UInteractiveTool* Tool, 
		TFunction<const UE::IInteractiveToolCommandsInterface&()>& GetterOut) const;
	// Used to unbind extension tool commands
	TFunction<const UE::IInteractiveToolCommandsInterface& ()> ExtensionToolCommandsGetter;
};
