// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API MODELINGTOOLSEDITORMODE_API

struct FSlateBrush;

class FModelingToolsEditorModeStyle
{
public:
	static UE_API void Initialize();

	static UE_API void Shutdown();

	static UE_API TSharedPtr< class ISlateStyle > Get();

	static UE_API FName GetStyleSetName();

	// use to access icons defined by the style set by name, eg GetBrush("BrushFalloffIcons.Smooth")
	static UE_API const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = NULL);

private:
	static UE_API FString InContent(const FString& RelativePath, const ANSICHAR* Extension);

private:
	static UE_API TSharedPtr< class FSlateStyleSet > StyleSet;
};

#undef UE_API
