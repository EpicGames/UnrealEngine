// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/BaseToolkit.h"
#include "StatusBarSubsystem.h"
#include "ToolHostCustomizationAPI.h" // FAcceptCancelButtonOverrideParams

class IAssetViewport;
class SEditableTextBox;
class STextComboBox;
namespace ETextCommit { enum Type : int; }

#define LOCTEXT_NAMESPACE "ModelingToolsEditorModeToolkit"

class STransformGizmoNumericalUIOverlay;
class IDetailsView;
class SButton;
class STextBlock;
class UGeometrySelectionManager;
class UInteractiveToolsPresetCollectionAsset;
class FRecentPresetCollectionProvider;
struct FAssetData;
class IToolStylusStateProviderAPI;

#if ENABLE_STYLUS_SUPPORT
namespace UE::Modeling
{
	class FStylusInputHandler;
}
#endif

struct FToolPresetOption
{
	FString PresetLabel;
	FString PresetTooltip;
	FSlateIcon PresetIcon;
	int32 PresetIndex;
	FSoftObjectPath PresetCollection;
};

class FModelingToolsEditorModeToolkit : public FModeToolkit
{
public:

	FModelingToolsEditorModeToolkit();
	~FModelingToolsEditorModeToolkit();
	
	/** FModeToolkit interface */
	virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual TSharedPtr<class SWidget> GetInlineContent() const override;

	// initialize toolkit widgets that need to wait until mode is initialized/entered
	virtual void InitializeAfterModeSetup();

	// set/clear notification message area
	virtual void PostNotification(const FText& Message);
	virtual void ClearNotification();

	// set/clear warning message area
	virtual void PostWarning(const FText& Message);
	virtual void ClearWarning();

	/** Returns the Mode specific tabs in the mode toolbar **/ 
	virtual void GetToolPaletteNames(TArray<FName>& InPaletteName) const;
	virtual FText GetToolPaletteDisplayName(FName PaletteName) const; 
	virtual void BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder);
	virtual void OnToolPaletteChanged(FName PaletteName) override;
	virtual bool HasIntegratedToolPalettes() const { return false; }
	virtual bool HasExclusiveToolPalettes() const { return false; }

	virtual FText GetActiveToolDisplayName() const override { return ActiveToolName; }
	virtual FText GetActiveToolMessage() const override { return ActiveToolMessage; }

	virtual void ShowRealtimeAndModeWarnings(bool bShowRealtimeWarning);

	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;

	virtual void CustomizeModeDetailsViewArgs(FDetailsViewArgs& ArgsInOut) override;

	void OnActiveViewportChanged(TSharedPtr<IAssetViewport>, TSharedPtr<IAssetViewport> );

	virtual void InvokeUI() override;
	virtual void ShutdownUI() override;
	virtual void ExtendSecondaryModeToolbar(UToolMenu *InModeToolbarMenu) override;

	virtual bool RequestAcceptCancelButtonOverride(IToolHostCustomizationAPI::FAcceptCancelButtonOverrideParams& Params);
	virtual bool RequestCompleteButtonOverride(IToolHostCustomizationAPI::FCompleteButtonOverrideParams& Params);
	virtual void ClearButtonOverrides();

	// Read and write Autogen Subfolder quick setting in restrictive mode.
	virtual FText GetRestrictiveModeAutoGeneratedAssetPathText() const;
	virtual void OnRestrictiveModeAutoGeneratedAssetPathTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit) const;

	virtual void ForceToolPaletteRebuild();

	// The mode is expected to call this after setting up the gizmo context object so that any subsequently
	// created gizmos are bound to the numerical UI.
	void BindGizmoNumericalUI();

	// get cached UGeometrySelectionManager pointer for the current Mode
	UGeometrySelectionManager* GetMeshSelectionManager();

	// @return true if we are currently in a tool, used to enable/disable UI things
	bool IsInActiveTool() const { return bInActiveTool; }

	// This is exposed only for the convenience of being able to create the numerical UI submenu
	// in a non-member function in ModelingModeToolkit_Toolbars.cpp
	TSharedPtr<STransformGizmoNumericalUIOverlay> GetGizmoNumericalUIOverlayWidget() { return GizmoNumericalUIOverlayWidget; }

	void NotifySelectionSystemEnabledStateModified();

	// Tells the stylus API to clean up any active tracked windows / contexts
	void DisconnectStylusStateProviderAPI();
	
	IToolStylusStateProviderAPI* GetStylusStateProviderAPI() const;

private:
	const static TArray<FName> PaletteNames_Standard;

	bool bInActiveTool = false;
	FText ActiveToolName;
	FText ActiveToolMessage;
	FStatusBarMessageHandle ActiveToolMessageHandle;
	const FSlateBrush* ActiveToolIcon = nullptr;

	TOptional<IToolHostCustomizationAPI::FAcceptCancelButtonOverrideParams> AcceptCancelButtonParams;
	TOptional<IToolHostCustomizationAPI::FCompleteButtonOverrideParams> CompleteButtonParams;
	bool bCurrentOverrideButtonsWereClicked = false;

	FReply HandleAcceptCancelClick(bool bAccept);
	FReply HandleCompleteClick();

	UGeometrySelectionManager* CachedSelectionManager = nullptr;

	TSharedPtr<SWidget> ToolkitWidget;
	void UpdateActiveToolProperties();
	void InvalidateCachedDetailPanelState(UObject* ChangedObject);

	/** A utility function to register the tool palettes with the ToolkitBuilder */
	void RegisterPalettes();
	FDelegateHandle ActivePaletteChangedHandle;

	TSharedPtr<SWidget> ToolShutdownViewportOverlayWidget;
	void MakeToolShutdownOverlayWidget();
	
	void MakeGizmoNumericalUIOverlayWidget();
	TSharedPtr<STransformGizmoNumericalUIOverlay> GizmoNumericalUIOverlayWidget;

	TSharedRef<SWidget> MakeMenu_ModelingModeConfigSettings();

	TSharedPtr<STextBlock> ModeWarningArea;
	TSharedPtr<STextBlock> ModeHeaderArea;
	TSharedPtr<STextBlock> ToolWarningArea;
	TSharedPtr<SButton> AcceptButton;
	TSharedPtr<SButton> CancelButton;
	TSharedPtr<SButton> CompletedButton;

	TSharedPtr<SWidget> MakeAssetConfigPanel();

	FText ActiveWarning{};

	TArray<TSharedPtr<FString>> AssetLocationModes;
	TArray<TSharedPtr<FString>> AssetSaveModes;
	TSharedPtr<STextComboBox> AssetLocationMode;
	TSharedPtr<STextComboBox> AssetSaveMode;
	TSharedPtr<SEditableTextBox> NewAssetPath;
	void UpdateAssetLocationMode(TSharedPtr<FString> NewString);
	void UpdateAssetSaveMode(TSharedPtr<FString> NewString);
	void UpdateAssetPanelFromSettings();
	void UpdateObjectCreationOptionsFromSettings();
	void OnAssetSettingsModified();
	FDelegateHandle AssetSettingsModifiedHandle;
	void OnShowAssetSettings();
	void SelectNewAssetPath() const;

	// Presets
	TSharedPtr<SWidget> ToolPresetArea;
	TSharedPtr<SWidget> MakePresetPanel();
	TSharedPtr<FAssetData> CurrentPreset;
	FSoftObjectPath CurrentPresetPath;
	FText CurrentPresetLabel;
	FToolPresetOption SelectedPresetToEdit;
	FString NewPresetLabel;
	FString NewPresetTooltip;
	FSlateIcon NewPresetIcon;

	TArray<TSharedPtr<FToolPresetOption>> AvailablePresetsForTool;
	TSharedPtr<SEditableComboBox<TSharedPtr<FString>>> PresetComboBox;
	TSharedPtr<FRecentPresetCollectionProvider> RecentPresetCollectionProvider;

	TSharedRef<SWidget> GetPresetCreateButtonContent();

	void CreateNewPresetInCollection(const FString& PresetLabel, FSoftObjectPath CollectionPath, const FString& ToolTip, FSlateIcon Icon);
	void LoadPresetFromCollection(const int32 PresetIndex, FSoftObjectPath CollectionPath);
	void UpdatePresetInCollection(const FToolPresetOption& PresetToEditIn, bool bUpdateStoredPresetValues);

	FString GetCurrentPresetPath() { return CurrentPresetPath.GetAssetPathString(); }
	void HandlePresetAssetChanged(const FAssetData& InAssetData);
	bool HandleFilterPresetAsset(const FAssetData& InAssetData);
	
	void RebuildPresetListForTool(bool bSettingsOpened);
	TSharedRef<SWidget> MakePresetComboWidget(TSharedPtr<FString> InItem);
	bool IsPresetEnabled() const;
	void ClearPresetComboList();

	/**
	 *  Updates the category button labels' visibility based off of the current editor preference for it
	 */
	void UpdateCategoryButtonLabelVisibility(UObject* Obj, FPropertyChangedEvent& ChangeEvent);

	/**
	 *  Updates the mesh element selection colors based off of the current editor preference for it
	 */
	void UpdateSelectionColors(UObject* Obj, FPropertyChangedEvent& ChangeEvent) const;

	TArray<TSharedPtr<FString>> AssetLODModes;
	TSharedPtr<STextBlock> AssetLODModeLabel;
	TSharedPtr<STextComboBox> AssetLODMode;
	TSharedPtr<FEditablePalette> FavoritesPalette;

	bool bFirstInitializeAfterModeSetup = true;
	bool bShowActiveSelectionActions = true;

#if ENABLE_STYLUS_SUPPORT
	TUniquePtr<UE::Modeling::FStylusInputHandler> StylusInputHandler;
#endif

	// tests if selection includes components backed by assets from the /Engine folder, of types the tools might otherwise modify (currently, static meshes)
	bool EngineAssetsSelected() const;

};

#undef LOCTEXT_NAMESPACE
