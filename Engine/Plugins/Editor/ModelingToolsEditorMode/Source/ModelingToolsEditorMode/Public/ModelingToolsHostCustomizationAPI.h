// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h" // TWeakPtr, TSharedRef
#include "ToolHostCustomizationAPI.h"
#include "UObject/Object.h"

#include "ModelingToolsHostCustomizationAPI.generated.h"

class FModelingToolsEditorModeToolkit;
class UInteractiveToolsContext;

/**
 * An implementation of IToolHostCustomizationAPI meant to be used with modeling tools editor mode.
 * Mostly it just forwards the relevant requests to the mode toolkit, which changes its accept/cancel/complete
 * overlay buttons temporarily (until the next click of those buttons).
 */
UCLASS(Transient, MinimalAPI)
class UModelingToolsHostCustomizationAPI : public UObject, public IToolHostCustomizationAPI
{
	GENERATED_BODY()

public:
	virtual bool SupportsAcceptCancelButtonOverride() override { return true; }
	virtual bool RequestAcceptCancelButtonOverride(FAcceptCancelButtonOverrideParams& Params) override;

	virtual bool SupportsCompleteButtonOverride() override { return true; }
	virtual bool RequestCompleteButtonOverride(FCompleteButtonOverrideParams& Params) override;

	virtual void ClearButtonOverrides() override;

	/** Helper to create, initialize, and register a UModelingToolsHostCustomizationAPI object with the context store */
	static UModelingToolsHostCustomizationAPI* Register(UInteractiveToolsContext* ToolsContext, 
		TSharedRef<FModelingToolsEditorModeToolkit> Toolkit);

	/** Helper to deregister the UModelingToolsHostCustomizationAPI from the context store */
	static bool Deregister(UInteractiveToolsContext* ToolsContext);

protected:
	TWeakPtr<FModelingToolsEditorModeToolkit> Toolkit;
};